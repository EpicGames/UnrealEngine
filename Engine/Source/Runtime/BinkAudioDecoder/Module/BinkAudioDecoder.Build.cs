// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using System.Collections.Generic;
using System.Security.Cryptography;
using UnrealBuildTool;

public class BinkAudioDecoder : ModuleRules
{
    // virtual so that NDA platforms can hide secrets
    protected virtual string GetLibrary(ReadOnlyTargetRules Target)
    {
        if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
        {
			if (Target.Architecture == UnrealArch.Arm64)
			{
				return Path.Combine(ModuleDirectory, "..", "SDK", "BinkAudio", "Lib", "binka_ue_decode_winarm64_static.lib");
			}
			return Path.Combine(ModuleDirectory, "..", "SDK", "BinkAudio", "Lib", "binka_ue_decode_win64_static.lib");
		}
		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Linux))
        {
            if (Target.Architecture == UnrealArch.Arm64)
            {
                return Path.Combine(ModuleDirectory, "..", "SDK", "BinkAudio", "Lib", "libbinka_ue_decode_lnxarm64_static.a");
            }
            if (Target.Architecture == UnrealArch.X64)
            {
                return Path.Combine(ModuleDirectory, "..", "SDK", "BinkAudio", "Lib", "libbinka_ue_decode_lnx64_static.a");
            }
        }
        if (Target.Platform == UnrealTargetPlatform.Mac)
        {
            return Path.Combine(ModuleDirectory, "..", "SDK", "BinkAudio", "Lib", "libbinka_ue_decode_osx_static.a");
        }
        if (Target.IsInPlatformGroup(UnrealPlatformGroup.IOS))
        {
			string LibExt = (Target.Architecture == UnrealArch.IOSSimulator) ? "_sim.a" : ".a";
            string PlatformName = Target.Platform.ToString().ToLower();
            return Path.Combine(ModuleDirectory, "..", "SDK", "BinkAudio", "Lib", $"libbinka_ue_decode_{PlatformName}_static{LibExt}");
        }
        return null;
    }

    public BinkAudioDecoder(ReadOnlyTargetRules Target) : base(Target)
    {
        ShortName = "BinkAudioDecoder";

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
                "CoreUObject",
                "Engine"
            }
        );

        PublicSystemIncludePaths.Add(Path.Combine(ModuleDirectory, "..", "SDK", "BinkAudio", "Include"));

		if (Target.Platform == UnrealTargetPlatform.Android)
		{
			// For Android each architecture is registered and unneeded one is filtered out
			PublicDefinitions.Add("WITH_BINK_AUDIO=1");
			PublicAdditionalLibraries.Add(Path.Combine(ModuleDirectory, "..", "SDK", "BinkAudio", "Lib", "Android", "ARM64", "libbinka_ue_decode_androidarm64_static.a"));
			PublicAdditionalLibraries.Add(Path.Combine(ModuleDirectory, "..", "SDK", "BinkAudio", "Lib", "Android", "x64", "libbinka_ue_decode_androidx64_static.a"));
		}
		else
		{
			string Library = GetLibrary(Target);
			if (Library != null)
			{
				// We have to have this because some audio mixers are shared across platforms
				// that we don't have libs for (e.g. hololens).
				PublicDefinitions.Add("WITH_BINK_AUDIO=1");
				PublicAdditionalLibraries.Add(Library);
			}
			else
			{
				PublicDefinitions.Add("WITH_BINK_AUDIO=0");
			}
		}
    }
}
