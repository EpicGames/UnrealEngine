// Copyright Epic Games, Inc. All Rights Reserved.

#include "BinkAudioInfo.h"
#include "Interfaces/IAudioFormat.h"

#include "Modules/ModuleInterface.h"

#include "binka_ue_file_header.h"
#include "binka_ue_decode.h"

#if !defined(PLATFORM_LITTLE_ENDIAN) || !PLATFORM_LITTLE_ENDIAN
#error "Bink Audio hasn't been updated for big endian."
#endif

DEFINE_LOG_CATEGORY_STATIC(LogBinkAudioDecoder, Log, All);

//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------

#define PTR_ADD(ptr,off) ((void*)(((uint8*)(ptr))+(off)))
#define Align32( val ) ( ( ( val ) + 31 ) & ~31 )


// This is shared in the encoder as the max number of bink audio streams. If
// you want to rebuild that, this can be otherwise be changed easily though
// it needs to be < 255
#define MAX_BINK_AUDIO_CHANNELS 16

namespace BinkAudioInfo
{
	static void LogOnce(BinkAudioFileHeader& Header)
	{
		static std::atomic<bool> bLogHeader = true;
		if (bLogHeader.exchange(false))
		{
			UE_LOG(LogBinkAudioDecoder, Log, TEXT("FBinkAudioInfo::ParseHeader: BEGIN LOG HEADER"));
			UE_LOGF(LogBinkAudioDecoder, Log, "Header.tag = %d", Header.tag);
			UE_LOGF(LogBinkAudioDecoder, Log, "Header.version = %d", Header.version);
			UE_LOGF(LogBinkAudioDecoder, Log, "Header.channels = %d", Header.channels);
			UE_LOGF(LogBinkAudioDecoder, Log, "Header.PADDING = %d", Header.PADDING);
			UE_LOGF(LogBinkAudioDecoder, Log, "Header.rate = %d", Header.rate);
			UE_LOGF(LogBinkAudioDecoder, Log, "Header.sample_count = %d", Header.sample_count);
			UE_LOGF(LogBinkAudioDecoder, Log, "Header.max_comp_space_needed = %d", Header.max_comp_space_needed);
			UE_LOGF(LogBinkAudioDecoder, Log, "Header.flags = %d", Header.flags);
			UE_LOGF(LogBinkAudioDecoder, Log, "Header.output_file_size = %d", Header.seek_table_entry_count);
			UE_LOGF(LogBinkAudioDecoder, Log, "Header.blocks_per_seek_table_entry = %d", Header.blocks_per_seek_table_entry);
			UE_LOG(LogBinkAudioDecoder, Log, TEXT("FBinkAudioInfo::ParseHeader: END LOG HEADER"));
		}
	}
}

//-----------------------------------------------------------------------------
//
// All memory for the decoder is in one contiguous block:
//
// BinkAudioDecoder structure
// StreamChannels
// Decoders
// OutputReservoir
// DeinterlaceBuffer
// SeekTable[SeekTableCount + 1] // see comment below
//
//-----------------------------------------------------------------------------
struct BinkAudioDecoder
{
	// # of low level bink audio streams (max 2 chans each)
	uint8 StreamCount;
	// where to copy the next output from
	uint32 OutputReservoirReadOffset;
	// # max size of the res.
	uint32 OutputReservoirTotalBytes;
	// offset to the seek table divided by 32.
	uint16 ToSeekTableOffset32;
	// # of entries in the seek table - the entries are byte sizes, so 
	// when decoding the seek table we convert to file offsets. This means
	// we actually have +1 entries in the decoded seek table due to the standard
	// "span -> edge" count thing.
	uint16 SeekTableCount;
	// # of low level bink blocks one seek table entry spans
	uint16 FramesPerSeekTableEntry;
	// # of frames we need to eat before outputting data due to a sample
	// accurate seek.
	uint16 ConsumeFrameCount;

	uint8* StreamChannels()
	{
		return (uint8*)(this + 1);
	}
	void** Decoders()
	{
		return (void**)PTR_ADD(this, Align32(sizeof(uint8) * StreamCount + sizeof(BinkAudioDecoder)));
	}
	uint32* SeekTable() // [SeekTableCount + 1] if SeekTableCount != 0, otherwise invalid.
	{
		return (uint32*)(PTR_ADD(this, ToSeekTableOffset32 * 32U));
	}
};

//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
FBinkAudioInfo::FBinkAudioInfo()
{
}

//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
FBinkAudioInfo::~FBinkAudioInfo()
{

}

//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
void FBinkAudioInfo::NotifySeek()
{
#if WITH_BINK_AUDIO
	UEBinkAudioDecodeInterface* BinkInterface = UnrealBinkAudioDecodeInterface();
	void** Streams = Decoder->Decoders();
	for (uint8 i = 0; i < Decoder->StreamCount; i++)
	{
		BinkInterface->ResetStartFn(Streams[i]);
	}
#endif // WITH_BINK_AUDIO
}

//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
void FBinkAudioInfo::SeekToTime(const float SeekTimeSeconds)
{
	NotifySeek();
	// If there's no seek table on the header, fall-back to Super implementation.
	if (Decoder->SeekTableCount == 0)
	{
		Super::SeekToTime(SeekTimeSeconds);
		return;
	}

	// no need to reset decoder here. Called in "SeekToFrame"

	// convert seconds to frames and call SeekToFrame
	uint32 SeekTimeFrames = 0;
	if (SeekTimeSeconds > 0)
	{
		SeekTimeFrames = (uint32)(SeekTimeSeconds * SampleRate);
	}

	SeekToFrame(SeekTimeFrames);
}

void FBinkAudioInfo::SeekToFrame(const uint32 InFrameNum)
{
	NotifySeek();
	// If there's no seek table on the header, fall-back to Super implementation.
	if (Decoder->SeekTableCount == 0)
	{
		Super::SeekToFrame(InFrameNum);
		return;
	}
	
	uint32 SeekTimeFrames = InFrameNum;
	uint32 SeekTimeSamples = SeekTimeFrames * NumChannels;

	uint32 SamplesInFrame = GetMaxFrameSizeSamples();
	if (SeekTimeSamples > this->TrueSampleCount)
	{
		SeekTimeSamples = this->TrueSampleCount - 1;
		SeekTimeFrames = (this->TrueSampleCount / NumChannels) - 1;
	}
	this->CurrentSampleCount = SeekTimeSamples;

	uint32 SamplesPerBlock = SamplesInFrame * Decoder->FramesPerSeekTableEntry;
	uint32 SeekTableIndex = SeekTimeFrames / SamplesPerBlock;
	uint32 SeekTableOffset = SeekTimeFrames % SamplesPerBlock;

	uint32 OffsetToBlock = Decoder->SeekTable()[SeekTableIndex] + sizeof(BinkAudioFileHeader) + Decoder->SeekTableCount * sizeof(uint16);

	Decoder->ConsumeFrameCount = SeekTableOffset;
	Decoder->OutputReservoirTotalBytes = 0;
	Decoder->OutputReservoirReadOffset = 0;

#if WITH_BINK_AUDIO
	UEBinkAudioDecodeInterface* BinkInterface = UnrealBinkAudioDecodeInterface();
	void** Streams = Decoder->Decoders();
	for (uint8 i = 0; i < Decoder->StreamCount; i++)
	{
		BinkInterface->ResetStartFn(Streams[i]);
	}
#endif // WITH_BINK_AUDIO

	//
	// Here we need to set up the data we get to point at the right spot.
	//
	if (StreamingSoundWaveData == nullptr)
	{
		// If we aren't streaming we can just go directly to the offset we need.
		this->SrcBufferOffset = OffsetToBlock;
	}
	else
	{
		const uint32 TotalStreamingChunks = StreamingSoundWaveData->GetNumChunks();
		uint32 RemnOffset = OffsetToBlock;

		// Find the chunk and offset to the block we need.
		for (uint32 BlockIndex = 0; BlockIndex < TotalStreamingChunks; ++BlockIndex)
		{
			const uint32 SizeOfChunk = StreamingSoundWaveData->GetSizeOfChunk(BlockIndex);

			if (SizeOfChunk > RemnOffset)
			{
				// This is the block we need
				// If we are in the current block *and* the current block doesn't need to be loaded,
				// only then can we set the block offset directly. This is because AudioDecompress.cpp
				// sets SrcBufferOffset to zero when switching to the next block.
				if (this->CurrentChunkIndex != BlockIndex ||
					this->SrcBufferData == nullptr)
				{
					// Need to seek to another block
					this->StreamSeekBlockIndex = BlockIndex;
					this->StreamSeekBlockOffset = RemnOffset;
				}
				else
				{
					// Seek within this block
					this->SrcBufferOffset = RemnOffset;
				}
				break;
			}

			RemnOffset -= SizeOfChunk;
		}
	}
}

//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
bool FBinkAudioInfo::ParseHeader(const uint8* InSrcBufferData, uint32 InSrcBufferDataSize, struct FSoundQualityInfo* QualityInfo)
{
	SrcBufferData = InSrcBufferData;
	SrcBufferDataSize = InSrcBufferDataSize;
	SrcBufferOffset = 0;
	CurrentSampleCount = 0;

	check(InSrcBufferDataSize >= sizeof(BinkAudioFileHeader));
	if (InSrcBufferDataSize < sizeof(BinkAudioFileHeader))
	{
		UE_LOGF(LogBinkAudioDecoder, Error, "FBinkAudioInfo::ParseHeader: Failed: InSrcBufferDataSize < sizeof(BinkAudioFileHeader) : %d < %llu", InSrcBufferDataSize, sizeof(BinkAudioFileHeader));
		return false;
	}

	BinkAudioFileHeader* Header = (BinkAudioFileHeader*)InSrcBufferData;
	if (Header->tag != 'UEBA')
	{
		UE_LOGF(LogBinkAudioDecoder, Error, "FBinkAudioInfo::ParseHeader: Failed: Header->tag != \'UEBA\' (%d): tag = %d", (uint32)('UEBA'), Header->tag);
		BinkAudioInfo::LogOnce(*Header);
		return false;
	}
	if (Header->version != 1)
	{
		UE_LOGF(LogBinkAudioDecoder, Error, "FBinkAudioInfo::ParseHeader: Failed: Header->version != 1: version = %d", Header->version);
		BinkAudioInfo::LogOnce(*Header);
		return false;
	}

	SampleRate = Header->rate;
	// Bink sample_count is per-channel so we multiply by num channels here
	TrueSampleCount = Header->sample_count * Header->channels;
	NumChannels = Header->channels;
	MaxCompSpaceNeeded = Header->max_comp_space_needed;
  
	uint32 SeekTableSize = Header->seek_table_entry_count * sizeof(uint16);
	if (sizeof(BinkAudioFileHeader) + SeekTableSize > InSrcBufferDataSize)
	{
		UE_LOGF(LogBinkAudioDecoder, Error, "BinkAudioInfo::ParseHeader Failed: Invalid SeekTableSize: SeekTableSize = %d; sizeof(BinkAudioFileHeader) = %llu; InSrcBufferDataSize = %d", SeekTableSize, SeekTableSize + sizeof(BinkAudioFileHeader), InSrcBufferDataSize);
		BinkAudioInfo::LogOnce(*Header);
		return false;
	}

	// Store the offset to where the audio data begins
	AudioDataOffset = sizeof(BinkAudioFileHeader) + SeekTableSize;

	// Write out the the header info
	if (QualityInfo)
	{
		QualityInfo->SampleRate = Header->rate;
		QualityInfo->NumChannels = Header->channels;
		QualityInfo->SampleDataSize = Header->sample_count * QualityInfo->NumChannels * sizeof(int16);
		QualityInfo->Duration = (float)Header->sample_count / QualityInfo->SampleRate;
	}

	return true;
}

//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
bool FBinkAudioInfo::CreateDecoder()
{
	UEBinkAudioDecodeInterface* BinkInterface = nullptr;

#if WITH_BINK_AUDIO
	BinkInterface = UnrealBinkAudioDecodeInterface();
#endif // WITH_BINK_AUDIO

	if (BinkInterface == nullptr)
	{
		UE_LOGF(LogBinkAudioDecoder, Error, "BinkAudioInfo::CreateDecoder Failed: BinkInterface does not exist for the current configuration");
		return false; // only happens if we dont have libs.
	}

	BinkAudioFileHeader* Header = (BinkAudioFileHeader*)SrcBufferData;

	// Bink is max stereo per stream
	uint32 StreamCount = (NumChannels + 1) >> 1;	

	// Figure memory for buffers:

	// Space for the decoder state
	uint32 DecoderMemoryTotal = 0;	
	uint32 DecoderMemoryPerStream[MAX_BINK_AUDIO_CHANNELS / 2];
	{
		uint32 RemnChannels = NumChannels;
		for (uint32 StreamIndex = 0; StreamIndex < StreamCount; StreamIndex++)
		{
			uint32 StreamChannels = RemnChannels;
			if (StreamChannels > 2)
			{
				StreamChannels = 2;
			}
			RemnChannels -= StreamChannels;
			DecoderMemoryPerStream[StreamIndex] = BinkInterface->MemoryFn(SampleRate, StreamChannels);
			DecoderMemoryTotal += DecoderMemoryPerStream[StreamIndex];
		}
	}
	
	// Space for the decoder pointers
	uint32 PtrMemory = Align32(sizeof(void*) * StreamCount);
	
	// Space for ourselves + the channel count for each stream.
	uint32 StructMemory = Align32(sizeof(BinkAudioDecoder) + sizeof(uint8)*StreamCount);

	// Space for decoded seek table
	uint32 SeekTableMemory = 0;
	if (Header->seek_table_entry_count)
	{
		SeekTableMemory = Align32(sizeof(uint32) * (Header->seek_table_entry_count + 1));
	}

	uint32 TotalMemory = DecoderMemoryTotal + PtrMemory + StructMemory + SeekTableMemory;

	//
	// Allocate and save offsets
	//
	RawMemory.AddZeroed(TotalMemory);	
	Decoder = (BinkAudioDecoder*)RawMemory.GetData();

	Decoder->StreamCount = StreamCount;
	Decoder->SeekTableCount = Header->seek_table_entry_count;
	Decoder->FramesPerSeekTableEntry = Header->blocks_per_seek_table_entry;

	// See layout discussion in class declaration
	void** Decoders = Decoder->Decoders();
	uint8* CurrentMemory = (uint8*)PTR_ADD(Decoders, PtrMemory);
	uint8* Channels = Decoder->StreamChannels();

	// Init decoders
	{
		uint8 RemnChannels = NumChannels;
		for (uint32 StreamIndex = 0; StreamIndex < StreamCount; StreamIndex++)
		{
			uint32 StreamChannels = RemnChannels;
			if (StreamChannels > 2)
			{
				StreamChannels = 2;
			}
			RemnChannels -= StreamChannels;

			Channels[StreamIndex] = StreamChannels;
			Decoders[StreamIndex] = (void**)CurrentMemory;
			CurrentMemory += DecoderMemoryPerStream[StreamIndex];
			BinkInterface->OpenFn(Decoders[StreamIndex], SampleRate, StreamChannels, true, true);
		}
	}

	Decoder->ToSeekTableOffset32 = (uint16)((CurrentMemory - RawMemory.GetData()) / 32);
	CurrentMemory += SeekTableMemory;

	// Decode the seek table
	if (Decoder->SeekTableCount)
	{
		uint32* SeekTable = Decoder->SeekTable();
		uint16* EncodedSeekTable = (uint16*)(SrcBufferData + sizeof(BinkAudioFileHeader));
		uint32 CurrentSeekOffset = 0;

		// the seek table has deltas from last, and we want absolutes
		for (uint32 i = 0; i < Decoder->SeekTableCount; i++)
		{
			SeekTable[i] = CurrentSeekOffset;
			CurrentSeekOffset += EncodedSeekTable[i];
		}
		SeekTable[Decoder->SeekTableCount] = CurrentSeekOffset;
	}

	SrcBufferOffset = sizeof(BinkAudioFileHeader) + Header->seek_table_entry_count * sizeof(uint16);
	return true;
}

//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
int32 FBinkAudioInfo::GetFrameSize()
{
	uint32 BlockSize;
	if (BinkAudioBlockSize(MaxCompSpaceNeeded, SrcBufferData + SrcBufferOffset, SrcBufferDataSize - SrcBufferOffset, &BlockSize) == false)
	{
		// Flag this as error so that the owning logic can clean up this decode.
		bErrorStateLatch = true;
		
		// Either malformed data, or not enough data.
		return 0;
	}

	return (int32)BlockSize;
}

//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
uint32 FBinkAudioInfo::GetMaxFrameSizeSamples() const
{
    if (SampleRate >= 44100) 
    {
        return 1920;
    }
    else if (SampleRate >= 22050) 
    {
        return 960;
    }
    else 
    {
        return 480;
    }
}

//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
FDecodeResult FBinkAudioInfo::Decode(const uint8* CompressedData, const int32 CompressedDataSize, uint8* OutPCMData, const int32 OutputPCMDataSize)
{
	UEBinkAudioDecodeInterface* BinkInterface = nullptr;

#if WITH_BINK_AUDIO
	BinkInterface = UnrealBinkAudioDecodeInterface();
#endif // WITH_BINK_AUDIO

	if (BinkInterface == nullptr)
	{
		return FDecodeResult(); // only happens with no libs.
	}

	// \todo consider a system wide shared buffer to prevent allocations and avoid holding memory needlessly.
	// ... also this is at most 7680 bytes, might be OK to just throw on the stack?
	TArray<int16, TAlignedHeapAllocator<16>> DeinterleavedDecodeBuffer;


	uint32 RemnOutputPCMDataSize = OutputPCMDataSize;
	uint32 RemnCompressedDataSize = CompressedDataSize;
	const uint8* CompressedDataEnd = CompressedData + CompressedDataSize;

	//
	// In the event we need to copy to a stack buffer, we alloca() it here so
	// that it's not inside the loop (for static analysis). We don't touch the memory until we need it
	// so it's just a couple instructions for the alloca().
	// (+8 for max block header size)
	uint8* StackBlockBuffer = (uint8*)alloca(MaxCompSpaceNeeded + BINK_UE_DECODER_END_INPUT_SPACE + 8);

	const uint32 DecodeFrames = GetMaxFrameSizeSamples();
	const uint32 FrameSize = NumChannels * sizeof(int16);
	const uint32 AllStreamsDecodeSize = DecodeFrames * FrameSize;
	while (RemnOutputPCMDataSize)
	{
		//
		// Drain the output reservoir before attempting a decode.
		//
		if (Decoder->OutputReservoirTotalBytes > Decoder->OutputReservoirReadOffset)
		{
			uint32 CopyBytes = Decoder->OutputReservoirTotalBytes - Decoder->OutputReservoirReadOffset;
			if (CopyBytes > RemnOutputPCMDataSize)
			{
				CopyBytes = RemnOutputPCMDataSize;
			}

			FMemory::Memcpy(OutPCMData, OutputReservoir.GetData() + Decoder->OutputReservoirReadOffset, CopyBytes);

			Decoder->OutputReservoirReadOffset += CopyBytes;

			RemnOutputPCMDataSize -= CopyBytes;
			OutPCMData += CopyBytes;

			if (RemnOutputPCMDataSize == 0)
			{
				// we filled entirely from the output reservoir
				break;
			}
		}

		if (RemnCompressedDataSize == 0)
		{
			// This is the normal termination condition if we are routing through the output res.
			break;
		}

		if (BinkAudioValidateBlock(MaxCompSpaceNeeded, CompressedData, RemnCompressedDataSize) != BINK_AUDIO_BLOCK_VALID)
		{
			// The splitting system should ensure that we only ever get complete blocks - so this is bizarre.
			UE_LOGF(LogBinkAudioDecoder, Warning, "Got weird buffer, validate returned %d", BinkAudioValidateBlock(MaxCompSpaceNeeded, CompressedData, RemnCompressedDataSize));
			break;
		}

		uint8 const* BlockStart = nullptr;
		uint8 const* BlockEnd = nullptr;
		uint32 TrimToFrameCount = 0;
		BinkAudioCrackBlock(CompressedData, &BlockStart, &BlockEnd, &TrimToFrameCount);
		uint8 const* BlockBase = CompressedData;

		uint32 DecodeFramesThisBlock = DecodeFrames;
		if (TrimToFrameCount != ~0U)
		{
			if (TrimToFrameCount > DecodeFrames)
			{
				bErrorStateLatch = true;
				return FDecodeResult(); // corrupted - should never encode a trim LARGER than the block.
			}
			DecodeFramesThisBlock = TrimToFrameCount;
		}

		// If we are consuming more than the entire block, just advance without actually decoding.
		if (Decoder->ConsumeFrameCount >= DecodeFramesThisBlock)
		{
			uint32 InputConsumed = (uint32)(BlockEnd - CompressedData);
			CompressedData += InputConsumed;
			RemnCompressedDataSize -= InputConsumed;

			Decoder->ConsumeFrameCount -= DecodeFramesThisBlock;
			continue;
		}

		//
		// We need to make sure there's room available for Bink to read past the end
		// of the buffer (for vector decoding). If there's not, we need to copy to a
		// temp buffer.
		//
		bool HasRoomForDecode = (CompressedDataEnd - BlockEnd) > BINK_UE_DECODER_END_INPUT_SPACE;
		if (HasRoomForDecode == false)
		{
			// This looks weird, but in order for the advancement logic to work,
			// we need to replicate the entire block including the header.
			size_t BlockOffset = BlockStart - BlockBase;
			size_t BlockSize = BlockEnd - BlockBase;
			if (BlockSize > MaxCompSpaceNeeded + 8) // +8 for max block header size
			{
				UE_LOG(LogBinkAudioDecoder, Error, TEXT("BAD! Validated block exceeds header max block size (%" SIZE_T_FMT " vs %d)"), BlockSize, MaxCompSpaceNeeded);
				bErrorStateLatch = true;
				break;
			}

			FMemory::Memcpy(StackBlockBuffer, BlockBase, BlockSize);

			// this is technically not needed, but just so that any analysis shows that
			// we've initialized all the memory we touch.
			FMemory::Memset(StackBlockBuffer + BlockSize, 0, BINK_UE_DECODER_END_INPUT_SPACE);

			BlockBase = StackBlockBuffer;
			BlockStart = StackBlockBuffer + BlockOffset;
			BlockEnd = StackBlockBuffer + BlockSize;
		}


		//
		// If we're a simple single stream and we have enough output space,
		// just decode directly into our destination to avoid some
		// copies.
		//
		// We also have to have a "simple" decode - i.e. aligned and no consume. This should be almost all
		// of the blocks in a mono/stereo streaming source.
		//
		if (Decoder->StreamCount == 1 &&
			AllStreamsDecodeSize <= RemnOutputPCMDataSize &&
			(((size_t)OutPCMData)&0xf) == 0 &&
			Decoder->ConsumeFrameCount == 0)
		{
			uint32 DecodedBytes = BinkInterface->DecodeFn(Decoder->Decoders()[0], OutPCMData, RemnOutputPCMDataSize, &BlockStart, BlockEnd);
			check (DecodedBytes == AllStreamsDecodeSize);
			if (DecodedBytes != AllStreamsDecodeSize)
			{
				bErrorStateLatch = true;
				return FDecodeResult(); // bink should always return full blocks
			}

			// Set to any trimmed value
			DecodedBytes = DecodeFramesThisBlock * FrameSize;

			if (BlockStart != BlockEnd)
			{
				// Header mismatch? We should always consume exactly what we expected to.
				UE_LOGF(LogBinkAudioDecoder, Error, "BinkAudio consumed unexpected amount! BlockEnd = 0x%llx BlockStart = 0x%llx BlockBase = 0x%llx",
					(uint64)BlockEnd, (uint64)BlockStart, (uint64)BlockBase);
				bErrorStateLatch = true;
				return FDecodeResult();
			}

			uint32 InputConsumed = (uint32)(BlockStart - BlockBase);

			OutPCMData += DecodedBytes;
			CompressedData += InputConsumed;
			RemnCompressedDataSize -= InputConsumed;
			RemnOutputPCMDataSize -= DecodedBytes;
			continue;
		}

		// Otherwise, we go into a buffer for deinterlacing / trimming / whatever.
		// We interlace each stream so we only ever need stereo space here.
		DeinterleavedDecodeBuffer.SetNumUninitialized(DecodeFrames * 2);

		// Bink always emits full transform blocks so we can do our frame trimming up
		// front.
		uint32 DecodedFramesStart = 0;
		uint32 DecodedFramesEnd = DecodeFramesThisBlock;

		uint32 FramesAvailable = DecodedFramesEnd - DecodedFramesStart;

		// Check if we need to eat some frames due to a sample-accurate seek.
		if (Decoder->ConsumeFrameCount)
		{
			uint32 ConsumedFrameCount = FMath::Min((uint32)Decoder->ConsumeFrameCount, FramesAvailable);
			DecodedFramesStart += ConsumedFrameCount;
			FramesAvailable = DecodedFramesEnd - DecodedFramesStart;
			Decoder->ConsumeFrameCount -= ConsumedFrameCount;
		}

		// Check if we can deinterleave directly to the output
		int16* InterleaveDestination = (int16*)OutPCMData;

		// Since we are interlacing into this destination we only need what we use,
		// not the full decode.
		bool bDirectDecode = true;
		if (FramesAvailable * FrameSize <= RemnOutputPCMDataSize)
		{
			// Yay - can just go direct and not even allocate the output reservoir.
		}
		else
		{
			// We have to go through the output reservoir! We allocate enough for any run we do
			// since it'll just stay allocated (once we use it once, we assume we'll continue to need
			// it due to the circumstances when it's needed in UE).
			OutputReservoir.SetNumUninitialized(DecodeFrames * FrameSize);
			InterleaveDestination = (int16*)OutputReservoir.GetData();
			bDirectDecode = false;
		}

		uint32 ChannelIndex = 0;
		for (uint32 i = 0, StreamChannels = Decoder->StreamChannels()[i]; i < Decoder->StreamCount; i++, ChannelIndex += StreamChannels)
		{
			// Decode into the channel's slot in the deinterlace buffer.
			uint32 DecodedBytes = BinkInterface->DecodeFn(
				Decoder->Decoders()[i],
				(uint8*)DeinterleavedDecodeBuffer.GetData(), 
				DecodeFrames * sizeof(int16) * StreamChannels,
				&BlockStart, BlockEnd);

			check(DecodedBytes == DecodeFrames * StreamChannels * sizeof(int16));
			if (DecodedBytes != DecodeFrames * StreamChannels * sizeof(int16))
			{
				// must decode entire blocks.
				bErrorStateLatch = true;
				return FDecodeResult();
			}

			// Bink audio emits pairs interlaced already, so we can copy stereo as 4 bytes
			if (StreamChannels == 2)
			{
				const uint32* InBuffer = ((uint32*)DeinterleavedDecodeBuffer.GetData()) + DecodedFramesStart;
				uint32* ChannelInterleaveDestination = (uint32*)(InterleaveDestination + ChannelIndex);
				for (uint32 SampleIdx = 0; SampleIdx < FramesAvailable; SampleIdx++)
				{
					// we know this is aligned because we pack stereo up front.
					// not sure if there's room for SIMD here. If it's quad we could load 4 values, mask off
					// every other and OR in... seems like that's probably worse than just copying 4 bytes.
					// Might be able to get something with unwinding?
					*(uint32*)(InterleaveDestination + (ChannelIndex + (SampleIdx * NumChannels))) = InBuffer[SampleIdx];
				}
			}
			else
			{
				// This will almost never get hit - odd multichannel isn't common.
				const int16* InBuffer = ((int16*)DeinterleavedDecodeBuffer.GetData()) + DecodedFramesStart;
				for (uint32 SampleIdx = 0; SampleIdx < FramesAvailable; SampleIdx++)
				{
					InterleaveDestination[ChannelIndex + (SampleIdx * NumChannels)] = InBuffer[SampleIdx];
				}
			}
		}

		uint32 InputConsumed = (uint32)(BlockStart - BlockBase);
		CompressedData += InputConsumed;
		RemnCompressedDataSize -= InputConsumed;
		
		if (bDirectDecode)
		{
			// we need to update the dest pointer since we went direct.
			OutPCMData += FramesAvailable * FrameSize;
			RemnOutputPCMDataSize -= FramesAvailable * FrameSize;
		}
		else
		{
			Decoder->OutputReservoirTotalBytes = FramesAvailable * FrameSize;
			Decoder->OutputReservoirReadOffset = 0;
			// Fall through to the next loop to copy the decoded pcm data out of the reservoir.
		}
	} // while need output pcm data

	// We get here if we filled the output buffer or not.
	FDecodeResult Result;
	Result.NumPcmBytesProduced = OutputPCMDataSize - RemnOutputPCMDataSize;
	Result.NumAudioFramesProduced = Result.NumPcmBytesProduced / (sizeof(int16) * NumChannels);	
	Result.NumCompressedBytesConsumed = CompressedDataSize - RemnCompressedDataSize;
	return Result;
}

bool FBinkAudioInfo::HasError() const
{
	return Super::HasError() || bErrorStateLatch;
}

class BINKAUDIODECODER_API FBinkAudioDecoderModule : public IModuleInterface
{
public:	
	TUniquePtr<IAudioInfoFactory> Factory;

	virtual void StartupModule() override
	{	
		Factory = MakeUnique<FSimpleAudioInfoFactory>([] { return new FBinkAudioInfo(); }, Audio::NAME_BINKA);
	}

	virtual void ShutdownModule() override {}
};

IMPLEMENT_MODULE(FBinkAudioDecoderModule, BinkAudioDecoder)
