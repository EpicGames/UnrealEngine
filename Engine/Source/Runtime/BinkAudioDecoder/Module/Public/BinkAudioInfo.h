// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AudioDecompress.h"

/**
*	Decoder thunk to the Bink Audio libraries. Also manages file parsing
*	for the cooked data.
*/
class  FBinkAudioInfo : public IStreamedCompressedInfo
{
public:
	BINKAUDIODECODER_API FBinkAudioInfo();
	BINKAUDIODECODER_API virtual ~FBinkAudioInfo();

	//~ Begin IStreamedCompressedInfo Interface
	virtual bool ParseHeader(const uint8* InSrcBufferData, uint32 InSrcBufferDataSize, FSoundQualityInfo* QualityInfo) override;
	virtual int32 GetFrameSize() override;
	virtual uint32 GetMaxFrameSizeSamples() const override;
	virtual bool CreateDecoder() override;
	virtual void SeekToTime(const float SeekToTimeSeconds) override;
	virtual void SeekToFrame(const uint32 SeekTimeFrames) override;
	virtual FDecodeResult Decode(const uint8* CompressedData, const int32 CompressedDataSize, uint8* OutPCMData, const int32 OutputPCMDataSize) override;
	virtual bool HasError() const override;
	//~ End IStreamedCompressedInfo Interface

protected:
	using Super = IStreamedCompressedInfo;
	
	void NotifySeek();
	
	// copied from header during ParseHeader
	uint32 MaxCompSpaceNeeded;
	uint32 SampleRate;

	// we lazy init this because the general use case (streaming) should always be able to decode
	// directly to the output buffer.
	TArray<uint8, TAlignedHeapAllocator<16>> OutputReservoir;
	struct BinkAudioDecoder* Decoder = 0;
	TArray<uint8, TAlignedHeapAllocator<16>> RawMemory;
	bool bErrorStateLatch = false;
};
