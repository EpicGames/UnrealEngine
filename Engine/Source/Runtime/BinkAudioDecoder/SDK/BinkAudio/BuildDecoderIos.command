#!/bin/bash

set -e

UE_MODULE_LOCATION=`pwd`

SOURCE_LOCATION="$UE_MODULE_LOCATION/Src"
BUILD_LOCATION="$UE_MODULE_LOCATION/Intermediate"
OUTPUT_LOCATION="$UE_MODULE_LOCATION/Lib"

if [[ ! -w $OUTPUT_LOCATION/libbinka_ue_decode_ios_static.a ]]; then
	echo "Check out $OUTPUT_LOCATION/libbinka_ue_decode_ios_static.a"
	exit 1
fi

rm -rf $BUILD_LOCATION
mkdir $BUILD_LOCATION
mkdir $BUILD_LOCATION/ios
mkdir $BUILD_LOCATION/ios/arm64
mkdir $BUILD_LOCATION/ios/arm7
mkdir $BUILD_LOCATION/ios/arm7s
mkdir $BUILD_LOCATION/ios/x64
mkdir $BUILD_LOCATION/ios/sim

SIM_ARGS=(
	-arch x86_64
	--target=x86_64-apple-ios10
	-mmmx
	-isysroot /Applications/Xcode.app/Contents/Developer/Platforms/iPhoneSimulator.platform/Developer/SDKs/iPhoneSimulator.sdk
)

SIM_ARGS_ARM64=(
	-arch arm64
	-miphonesimulator-version-min=15
	-isysroot /Applications/Xcode.app/Contents/Developer/Platforms/iPhoneSimulator.platform/Developer/SDKs/iPhoneSimulator.sdk
)

ARM64_ARGS=(
	-arch arm64
	--target=arm-apple-ios10
	-isysroot /Applications/Xcode.app/Contents/Developer/Platforms/iPhoneOS.platform/Developer/SDKs/iPhoneOS.sdk
)

ARM7_ARGS=(
	-arch armv7
	--target=arm-apple-ios10
	-isysroot /Applications/Xcode.app/Contents/Developer/Platforms/iPhoneOS.platform/Developer/SDKs/iPhoneOS.sdk
)
ARM7S_ARGS=(
	-arch armv7s
	--target=arm-apple-ios10
	-isysroot /Applications/Xcode.app/Contents/Developer/Platforms/iPhoneOS.platform/Developer/SDKs/iPhoneOS.sdk
)
COMMON_ARGS=(
	-c
	-D__GCC__
	-DNDEBUG
	-D__RADINSTATICLIB__
	-DWRAP_PUBLICS=BACDUE # Prefix symbols so that mulitple libs using the same source dont get reused by the linker
	-ffp-contract=off # Prevent FMA contraction for consistency between arm/x64
	-fno-exceptions
	-fno-omit-frame-pointer
	-fno-rtti
	-fno-strict-aliasing # prevent optimizations from introducing random silent bugs
	-fno-unroll-loops
	-fno-vectorize # Weve vectorized everything so this just makes the tail computation get unrolled unnecessarily
	-fvisibility=hidden
	-ggdb
	-mllvm
	-inline-threshold=64 # Pass inline threhold to llvm to prevent binary size explosion
	-momit-leaf-frame-pointer
	-O2
	-IInclude
)

SOURCES=(
	binkacd.cpp
	binka_ue_decode.cpp
	radfft.cpp
)


OUTPUT=()
for source_file in "${SOURCES[@]}"
do
	clang ${ARM64_ARGS[@]} ${COMMON_ARGS[@]} -o $BUILD_LOCATION/ios/arm64/${source_file%.*}.o $SOURCE_LOCATION/$source_file
	OUTPUT+=( $BUILD_LOCATION/ios/arm64/${source_file%.*}.o )
done
ar rcs $OUTPUT_LOCATION/libbinka_ue_decode_ios_arm64_static.a ${OUTPUT[@]}

OUTPUT=()
for source_file in "${SOURCES[@]}"
do
	clang ${ARM7_ARGS[@]} ${COMMON_ARGS[@]} -o $BUILD_LOCATION/ios/arm7/${source_file%.*}.o $SOURCE_LOCATION/$source_file
	OUTPUT+=( $BUILD_LOCATION/ios/arm7/${source_file%.*}.o )
done
ar rcs $OUTPUT_LOCATION/libbinka_ue_decode_ios_arm7_static.a ${OUTPUT[@]}

OUTPUT=()
for source_file in "${SOURCES[@]}"
do
	clang ${ARM7S_ARGS[@]} ${COMMON_ARGS[@]} -o $BUILD_LOCATION/ios/arm7s/${source_file%.*}.o $SOURCE_LOCATION/$source_file
	OUTPUT+=( $BUILD_LOCATION/ios/arm7s/${source_file%.*}.o )
done
ar rcs $OUTPUT_LOCATION/libbinka_ue_decode_ios_arm7s_static.a ${OUTPUT[@]}
OUTPUT=()

for source_file in "${SOURCES[@]}"
do
	clang ${SIM_ARGS_ARM64[@]} ${COMMON_ARGS[@]} -o $BUILD_LOCATION/ios/sim/${source_file%.*}.o $SOURCE_LOCATION/$source_file
	OUTPUT+=( $BUILD_LOCATION/ios/sim/${source_file%.*}.o )
done
ar rcs $OUTPUT_LOCATION/libbinka_ue_decode_ios_static_sim.a ${OUTPUT[@]}
OUTPUT=()

SOURCES+=(x86_cpu.c)
for source_file in "${SOURCES[@]}"
do
	clang ${SIM_ARGS[@]} ${COMMON_ARGS[@]} -o $BUILD_LOCATION/ios/x64/${source_file%.*}.o $SOURCE_LOCATION/$source_file
	OUTPUT+=( $BUILD_LOCATION/ios/x64/${source_file%.*}.o )
done

ar rcs $OUTPUT_LOCATION/libbinka_ue_decode_ios_x64_static.a ${OUTPUT[@]}

lipo -create $OUTPUT_LOCATION/libbinka_ue_decode_ios_x64_static.a $OUTPUT_LOCATION/libbinka_ue_decode_ios_arm64_static.a $OUTPUT_LOCATION/libbinka_ue_decode_ios_arm7_static.a $OUTPUT_LOCATION/libbinka_ue_decode_ios_arm7s_static.a -output $OUTPUT_LOCATION/libbinka_ue_decode_ios_static.a
rm $OUTPUT_LOCATION/libbinka_ue_decode_ios_x64_static.a $OUTPUT_LOCATION/libbinka_ue_decode_ios_arm64_static.a $OUTPUT_LOCATION/libbinka_ue_decode_ios_arm7_static.a $OUTPUT_LOCATION/libbinka_ue_decode_ios_arm7s_static.a

echo Done.
