#!/bin/bash

set -e

UE_MODULE_LOCATION=`pwd`

SOURCE_LOCATION="$UE_MODULE_LOCATION/Src"
BUILD_LOCATION="$UE_MODULE_LOCATION/Intermediate"
OUTPUT_LOCATION="$UE_MODULE_LOCATION/Lib"

if [[ ! -w $OUTPUT_LOCATION/libbinka_ue_decode_osx_static.a ]]; then
	echo "Check out $OUTPUT_LOCATION/libbinka_ue_decode_osx_static.a"
	exit 1
fi

rm -rf $BUILD_LOCATION
mkdir $BUILD_LOCATION
mkdir $BUILD_LOCATION/arm64
mkdir $BUILD_LOCATION/x64

X64_ARGS=(
	-arch x86_64
	-msse
	-msse2
	-msse3
	-mssse3
)

ARM64_ARGS=(
	-arch arm64
)

COMMON_ARGS=(
	-c
	-D__GCC__
	-D__MACHO__
	-DNDEBUG
	-D__RADINSTATICLIB__
	-DWRAP_PUBLICS=BACDUE # Prefix symbols so that mulitple libs using the same source dont get reused by the linker
	-ffp-contract=off # Prevent FMA contraction for consistency between arm/x64
	-fno-exceptions
	-fno-omit-frame-pointer
	-fno-rtti
	-fno-strict-aliasing # prevent optimizations from introducing random silent bugs
	-fno-unroll-loops
	-fno-vectorize # Weve vectorized everything so this just makes the tail computation get unrolled unnecessarily
	-fvisibility=hidden
	-ggdb
	-mllvm
	-inline-threshold=64 # Pass inline threhold to llvm to prevent binary size explosion
	-momit-leaf-frame-pointer
	-O2
	-IInclude
	-mmacosx-version-min=10.14
)

SOURCES=(
	binkacd.cpp
	x86_cpu.c
	binka_ue_decode.cpp
	radfft.cpp
)


for source_file in "${SOURCES[@]}"
do
	clang ${X64_ARGS[@]} ${COMMON_ARGS[@]} -o $BUILD_LOCATION/x64/${source_file%.*}.o $SOURCE_LOCATION/$source_file
	OUTPUT+=( $BUILD_LOCATION/x64/${source_file%.*}.o )
done

ar rcs $OUTPUT_LOCATION/libbinka_ue_decode_osx_x64_static.a ${OUTPUT[@]}

OUTPUT=()
for source_file in "${SOURCES[@]}"
do
	clang ${ARM64_ARGS[@]} ${COMMON_ARGS[@]} -o $BUILD_LOCATION/arm64/${source_file%.*}.o $SOURCE_LOCATION/$source_file
	OUTPUT+=( $BUILD_LOCATION/arm64/${source_file%.*}.o )
done
ar rcs $OUTPUT_LOCATION/libbinka_ue_decode_osx_arm64_static.a ${OUTPUT[@]}

lipo -create $OUTPUT_LOCATION/libbinka_ue_decode_osx_x64_static.a $OUTPUT_LOCATION/libbinka_ue_decode_osx_arm64_static.a -output $OUTPUT_LOCATION/libbinka_ue_decode_osx_static.a
rm $OUTPUT_LOCATION/libbinka_ue_decode_osx_arm64_static.a
rm $OUTPUT_LOCATION/libbinka_ue_decode_osx_x64_static.a

echo Done.
