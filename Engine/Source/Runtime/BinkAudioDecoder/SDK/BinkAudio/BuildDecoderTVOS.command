#!/bin/bash

set -e

UE_MODULE_LOCATION=`pwd`

SOURCE_LOCATION="$UE_MODULE_LOCATION/Src"
BUILD_LOCATION="$UE_MODULE_LOCATION/Intermediate"
OUTPUT_LOCATION="$UE_MODULE_LOCATION/Lib"

if [[ -w $OUTPUT_LOCATION/libbinka_ue_decode_tvos_static.a ]]; then
	echo "Check out $OUTPUT_LOCATION/libbinka_ue_decode_tvos_static.a"
	exit 1
fi

rm -rf $BUILD_LOCATION
mkdir $BUILD_LOCATION
mkdir $BUILD_LOCATION/tvos

ARM64_ARGS=(
	-arch arm64
	--target=arm64-apple-tvos15
	-isysroot /Applications/Xcode.app/Contents/Developer/Platforms/AppleTVOS.platform/Developer/SDKs/AppleTVOS.sdk
)

COMMON_ARGS=(
	-c
	-D__GCC__
	-DNDEBUG
	-D__RADINSTATICLIB__
	-DWRAP_PUBLICS=BACDUE # Prefix symbols so that mulitple libs using the same source dont get reused by the linker
	-ffp-contract=off # Prevent FMA contraction for consistency between arm/x64
	-fno-exceptions
	-fno-omit-frame-pointer
	-fno-rtti
	-fno-strict-aliasing # prevent optimizations from introducing random silent bugs
	-fno-unroll-loops
	-fno-vectorize # Weve vectorized everything so this just makes the tail computation get unrolled unnecessarily
	-fembed-bitcode
	-fvisibility=hidden
	--target=arm64-apple-tvos14
	-ggdb
	-momit-leaf-frame-pointer
	-O2
	-IInclude
)

SOURCES=(
	binkacd.cpp
	binka_ue_decode.cpp
	radfft.cpp
)


OUTPUT=()
for source_file in "${SOURCES[@]}"
do
	clang ${ARM64_ARGS[@]} ${COMMON_ARGS[@]} -o $BUILD_LOCATION/tvos/${source_file%.*}.o $SOURCE_LOCATION/$source_file
	OUTPUT+=( $BUILD_LOCATION/tvos/${source_file%.*}.o )
done
ar rcs $OUTPUT_LOCATION/libbinka_ue_decode_tvos_static.a ${OUTPUT[@]}

echo Done.
