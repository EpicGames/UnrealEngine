#!/bin/bash

set -e

UE_MODULE_LOCATION=`pwd`

SOURCE_LOCATION="$UE_MODULE_LOCATION/Src"
BUILD_LOCATION="$UE_MODULE_LOCATION/Intermediate"
OUTPUT_LOCATION="$UE_MODULE_LOCATION/Lib"

#if [[ ! -w $OUTPUT_LOCATION/libbinka_ue_decode_visionos_static.a ]]; then
#	echo "Check out $OUTPUT_LOCATION/libbinka_ue_decode_visionos_static.a"
#	exit 1
#fi

rm -rf $BUILD_LOCATION
mkdir $BUILD_LOCATION
mkdir $BUILD_LOCATION/visionos
mkdir $BUILD_LOCATION/visionos/arm64
mkdir $BUILD_LOCATION/visionos/sim

#	-mxrossimulator-version-min=1

SIM_ARGS_ARM64=(
	-arch arm64
	--target=arm-apple-xros1-simulator
	-isysroot /Applications/Xcode.app/Contents/Developer/Platforms/XRSimulator.platform/Developer/SDKs/XRSimulator.sdk
)

ARM64_ARGS=(
	-arch arm64
	--target=arm-apple-xros1
	-isysroot /Applications/Xcode.app/Contents/Developer/Platforms/XROS.platform/Developer/SDKs/XROS.sdk
)

COMMON_ARGS=(
	-c
	-D__GCC__
	-DNDEBUG
	-D__RADINSTATICLIB__
	-DWRAP_PUBLICS=BACDUE # Prefix symbols so that mulitple libs using the same source dont get reused by the linker
	-ffp-contract=off # Prevent FMA contraction for consistency between arm/x64
	-fno-exceptions
	-fno-omit-frame-pointer
	-fno-rtti
	-fno-strict-aliasing # prevent optimizations from introducing random silent bugs
	-fno-unroll-loops
	-fno-vectorize # Weve vectorized everything so this just makes the tail computation get unrolled unnecessarily
	-fvisibility=hidden
	-ggdb
	-mllvm
	-inline-threshold=64 # Pass inline threhold to llvm to prevent binary size explosion
	-momit-leaf-frame-pointer
	-O2
	-IInclude
)

SOURCES=(
	binkacd.cpp
	binka_ue_decode.cpp
	radfft.cpp
)


OUTPUT=()
for source_file in "${SOURCES[@]}"
do
	clang ${ARM64_ARGS[@]} ${COMMON_ARGS[@]} -o $BUILD_LOCATION/visionos/arm64/${source_file%.*}.o $SOURCE_LOCATION/$source_file
	OUTPUT+=( $BUILD_LOCATION/visionos/arm64/${source_file%.*}.o )
done
ar rcs $OUTPUT_LOCATION/libbinka_ue_decode_visionos_static.a ${OUTPUT[@]}

OUTPUT=()
for source_file in "${SOURCES[@]}"
do
	clang ${SIM_ARGS_ARM64[@]} ${COMMON_ARGS[@]} -o $BUILD_LOCATION/visionos/sim/${source_file%.*}.o $SOURCE_LOCATION/$source_file
	OUTPUT+=( $BUILD_LOCATION/visionos/sim/${source_file%.*}.o )
done
ar rcs $OUTPUT_LOCATION/libbinka_ue_decode_visionos_static_sim.a ${OUTPUT[@]}

echo Done.
