Bink Audio Decoder/Encoder Build Instructions

First, the necessary source files to pass to the compiler:

Encoder
    binkace.c
    popmal.c
    varbits.c
    binka_ue_encode.cpp
    radfft.cpp
    ranged_log.cpp

Decoder
    binkacd.cpp
    binka_ue_decode.cpp
    radfft.cpp

    If a PC platform:
    x86_cpu.c

The other source files are support for the test app.

Be sure to define __RADINSTATICLIB__ on the compiler line: -D__RADINSTATICLIB__
unless compiling a shared library, than use -D__RADINDLL__

The decoder needs -DWRAP_PUBLICS=UEBA in order to keep external symbols that
might be shared with other RAD code to be unique and avoid linker issues.

If a modern x86 console with AVX support, then -D__RADCONSOLE__ 

If it's an NDA platform then add -D__RAD_NDA_PLATFORM__=<platform>, where
<platform> is the NDA platform label. The best way to find this is to look
in the Engine\Platforms\<>\Source\ThirdParty\BinkAudio\Src directory
and get the relevant part from the header filenames.

