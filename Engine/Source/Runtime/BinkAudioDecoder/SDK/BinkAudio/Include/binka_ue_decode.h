// Copyright Epic Games, Inc. All Rights Reserved.
//
// This decodes _blocks_ of bink audio data, not files. Usage is
// to create a decoder that will decode a stream of mono or stereo
// audio data, and sequentially feed it blocks of data.
//
#pragma once
#include <stdint.h>

/*
    UEBinkAudioDecodeInterface* Interface = UnrealBinkAudioDecodeInterface();

    // Allocate the decoder
    U32 Bytes = Interface->MemoryFn(48000, 2);
    void* Decoder = malloc(Bytes);

    // Init the decoder memory. Only fails with bad params.
    Interface->OpenFn(Decoder, 48000, 2, 1);

    // This assumes the blocks are back-to-back (not true for on-disc bink audio files)
    char* CompressedData;
    char* CompressedDataEnd = CompressedData + CompressedDataLen;
    while (CompressedData < CompressedDataEnd)
    {
        char OutputBuffer[BinkAudioDecodeOutputMaxSize()];
        U32 OutputBufferValidLen = Interface->DecodeFn(Decoder, OutputBuffer, BinkAudioDecodeOutputMaxSize(), &CompressedData, CompressedDataEnd);

        if (OutputBufferValidLen == 0)
            break; // error condition (or if CompressedData doesn't move)

        // work with OutputBuffer.
        // CompressedData has been updated by DecodeFn to point at the next
        // data.
    }

    // Done.
    free(Decoder);
*/

// with invalid data, we can read past the InputBuffer during parsing.
#define BINK_UE_DECODER_EXTRA_INPUT_SPACE 72

// with _normal_ data, we can read past the InputBuffer by 16 bytes
// due to vector bit decoding. It's highly unlikely that it reads that far,
// however _some_ amount of reading is all but certain.
#define BINK_UE_DECODER_END_INPUT_SPACE 16

// For the given rate and channels, this is the memory required for the decoder
// structure. Max 2 channels. Rates above 48000 are pretty useless as high freqs
// are crushed.
typedef uint32_t    DecodeMemoryFnType(uint32_t rate, uint32_t chans);

// Initialize the decoder, returns 0 on invalid parameters. Unreal should always be encoding bink_audio_2
typedef uint32_t    DecodeOpenFnType(void* BinkAudioDecoderMemory, uint32_t rate, uint32_t chans, bool interleave_output, bool is_bink_audio_2);

// Decode a single block. InputBuffer is updated with the amount of compressed data consumed for the block.
// Output is 16 bit pcm, interleaved if OpenFn() specified interleave_output.
typedef uint32_t    DecodeFnType(void* BinkAudioDecoderMemory, uint8_t* OutputBuffer, uint32_t OutputBufferLen, uint8_t const** InputBuffer, uint8_t const* InputBufferEnd);

// Call this when seeking/looping, this clears the overlap buffers.
typedef void        DecodeResetStartFrameFnType(void* BinkAudioDecoderMemory);

typedef struct UEBinkAudioDecodeInterface
{
    DecodeMemoryFnType* MemoryFn;
    DecodeOpenFnType* OpenFn;
    DecodeFnType* DecodeFn;
    DecodeResetStartFrameFnType* ResetStartFn;
} UEBinkAudioDecodeInterface;

UEBinkAudioDecodeInterface* UnrealBinkAudioDecodeInterface();

// what is the maximum data that can be touched of the output for ANY Bink file
// OutputBufferLen never needs to be larger than this for 1 block.
#define BinkAudioDecodeOutputMaxSize() ( 2048 * sizeof(int16_t) * 2 )  // 2 is channels
