// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include <stdint.h>

typedef void* BAUECompressAllocFnType(uintptr_t ByteCount);
typedef void BAUECompressFreeFnType(void* Ptr);

#define BINKA_COMPRESS_SUCCESS 0
#define BINKA_COMPRESS_ERROR_CHANS 1
#define BINKA_COMPRESS_ERROR_SAMPLES 2
#define BINKA_COMPRESS_ERROR_RATE 3
#define BINKA_COMPRESS_ERROR_QUALITY 4
#define BINKA_COMPRESS_ERROR_ALLOCATORS 5
#define BINKA_COMPRESS_ERROR_OUTPUT 6
#define BINKA_COMPRESS_ERROR_SEEKTABLE 7
#define BINKA_COMPRESS_ERROR_SIZE 8

#define BINKA_MAX_CHANS_STR TEXT("16") // (8*2) MAX_STREAMS stereo
#define BINKA_MIN_RATE_STR TEXT("2 khz")
#define BINKA_MAX_RATE_STR TEXT("256 khz")

//
// Compresses a bink audio _file_. This is distinct from _blocks_, which is
// what binka_ue_decode handles.
//
// Compresses up to 16 channels. Sample rates above 48000 technically supported however
// it doesn't really add anything.
// Quality is between 0-9, with 0 being high quality. Below 4 is pretty bad.
// PcmData should be interleaved 16 bit pcm data.
// PcmDataLen is in bytes.
// OutData will be filled with a buffer allocated by MemAlloc that contains the
// compressed file data.
// SeekTableMaxEntries is used to cap the amount of disk space used on the seek table.
// Historically this has been capped at 4k, or 8k of data. If GenerateSeekTable is set,
// this should be at least 1k or so to be useful.
uint8_t UECompressBinkAudio(
    void* PcmData, 
    uint32_t PcmDataLen, 
    uint32_t PcmRate, 
    uint8_t PcmChannels, 
    uint8_t Quality, 
    uint8_t GenerateSeekTable, 
    uint16_t SeekTableMaxEntries,
    BAUECompressAllocFnType* MemAlloc,
    BAUECompressFreeFnType* MemFree,
    void** OutData, 
    uint32_t* OutDataLen);

