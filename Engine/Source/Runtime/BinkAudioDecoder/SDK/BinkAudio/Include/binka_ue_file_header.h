// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include <stdint.h>

typedef uint32_t uint32;
typedef uint16_t uint16;
typedef uint8_t uint8;

struct BinkAudioFileHeader
{
    uint32 tag; // UEBA
    uint8 version;
    uint8 channels;
    uint16 PADDING; // unused atm.
    uint32 rate;
    uint32 sample_count;

    // the max for a bink audio block - includes all streams, but does _not_ include
    // the block header info.
    uint16 max_comp_space_needed;

    // unused in UE (holds whether it's BA2 or not, which is always true for UE)
    uint16 flags;

    // bytes for the whole file
    uint32 output_file_size;

    // number of encoded seek table entries stored directly after the file header.
    // each one is 16 bits, and is the delta from the last entry.
    uint16 seek_table_entry_count;

    // number of bink audio blocks for a single entry in the seek table. Encoding
    // keeps seek tables under 4k, so larger files will correspondingly have more
    // distance between seek table lookup points. Note that while you could consider
    // each bink audio stream decode a single block, this is referring to the set
    // of low level bink audio blocks for a given timepoint (so for 6 channels, you
    // have 3 low level bink streams, but the set of those 3 is referred to as one
    // block for the purposes of this.)
    uint16 blocks_per_seek_table_entry;
};

#define BLOCK_HEADER_MAGIC 0x9999

static bool BinkAudioBlockSize(uint32 header_max_comp_space_needed, uint8 const* input_reservoir, uint32 input_reservoir_len, uint32* out_block_size)
{
    //
    // Returns true if the current input buffer can be decoded
    //
    if (input_reservoir_len < 4)
    {
        // need an unknown amount of data - fill the reservoir
        return false;
    }

    // Check for the frame header.
    uint32 FrameHeaderSize = 4;
    uint32 FrameCheck = *(uint32*)(input_reservoir);

    if ((FrameCheck & 0xffff) != BLOCK_HEADER_MAGIC)
    {
        return false;
    }

    uint32 FrameSize = FrameCheck >> 16;
    if (FrameSize == 0xffff)
    {
        // Trimmed frame - size & sample next.
        if (input_reservoir_len < 8)
        {
            return false;
        }

        uint32 TrimmedFrameHeader = *(uint32*)(input_reservoir + 4);
        FrameSize = TrimmedFrameHeader & 0xffff;
        FrameHeaderSize = 8;
    }

     if (FrameSize > header_max_comp_space_needed)
     {
         // Invalid frame.
         return false;
     }

    *out_block_size = FrameSize + FrameHeaderSize;
    return true;
}

#define BINK_AUDIO_BLOCK_VALID 0            // The buffer can be passed to Bink.
#define BINK_AUDIO_BLOCK_INCOMPLETE 1       // The buffer looks good but needs more data
#define BINK_AUDIO_BLOCK_INVALID 2          // Buffer consistency checks failed.
static uint8 BinkAudioValidateBlock(uint32 header_max_comp_space_needed, uint8 const* input_reservoir, uint32 input_reservoir_len)
{
    //
    // Returns true if the current input buffer can be decoded,
    // doesn't affect BcfStr at all.
    //
    uint8 const* input_buffer = input_reservoir;
    uint8 const* input_end = input_reservoir + input_reservoir_len;

    if (input_reservoir_len < 4)
        return BINK_AUDIO_BLOCK_INCOMPLETE; // no space for header.

    // Check for the frame header.
    uint32 FrameCheck = *(uint32 const*)(input_buffer);
    input_buffer += 4;
    if ((FrameCheck & 0xffff) != BLOCK_HEADER_MAGIC)
    {
        return BINK_AUDIO_BLOCK_INVALID;
    }

    uint32 FrameSize = FrameCheck >> 16;
    if (FrameSize == 0xffff)
    {
        // Trimmed frame - size & sample next.
        if (input_reservoir_len < 8)
            return BINK_AUDIO_BLOCK_INCOMPLETE; // no space for header

        uint32 TrimmedFrameHeader = *(uint32 const*)(input_buffer);
        input_buffer += 4;
        FrameSize = TrimmedFrameHeader & 0xffff;
    }

    if (FrameSize > header_max_comp_space_needed)
    {
        // Invalid frame.
        return BINK_AUDIO_BLOCK_INVALID;
    }

    uint8 const* frame_end = input_buffer + FrameSize;
    if (frame_end > input_end)
        return BINK_AUDIO_BLOCK_INCOMPLETE; // frame isn't resident

    if (frame_end == input_end)
        return BINK_AUDIO_BLOCK_VALID; // we assume it's valid, even though it's possible that it's a fake.

    if (frame_end + 4 <= input_end)
    {
        // we have enough to double check.
        if ((*(uint32 const*)(frame_end) & 0xffff) != BLOCK_HEADER_MAGIC)
        {
            // This was a fake!
            return BINK_AUDIO_BLOCK_INVALID;
        }
    }

    return BINK_AUDIO_BLOCK_VALID;
}

// input_reservoir MUST have passed BinkAudioValidateBlock
static void BinkAudioCrackBlock(uint8 const* input_reservoir, uint8 const** out_block_start, uint8 const** out_block_end, uint32* out_trim_to_sample_count)
{
    uint8 const* input_buffer = (uint8 const*)input_reservoir;
    uint32 FrameCheck = *(uint32 const*)(input_buffer);
    input_buffer += 4;

    uint32 TrimToSampleCount = ~0U;
    uint32 FrameBytes = FrameCheck >> 16;
    if (FrameBytes == 0xffff)
    {
        // Trimmed frame - # of output samples after.
        uint32 TrimmedFrameHeader = *(uint32 const*)(input_buffer);
        input_buffer += 4;

        FrameBytes = TrimmedFrameHeader & 0xffff;
        TrimToSampleCount = TrimmedFrameHeader >> 16;
    }

    *out_block_start = input_buffer;
    *out_block_end = input_buffer + FrameBytes;
    *out_trim_to_sample_count = TrimToSampleCount;
}