// Copyright Epic Games, Inc. All Rights Reserved.
#include "ticks.h"

#include <sys/time.h>
#include <unistd.h>
#include <sched.h>

U64 baue_ticks( void )
{
  struct timeval t;
  gettimeofday(&t, 0);
  return t.tv_sec * 1000000ULL + t.tv_usec;
}