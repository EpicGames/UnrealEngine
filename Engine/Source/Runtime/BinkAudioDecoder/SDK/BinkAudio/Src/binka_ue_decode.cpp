// Copyright Epic Games, Inc. All Rights Reserved.

#include "binkacd.h"
#include "binka_ue_decode.h"

// Amount of memory to pass to DecompressOpen
static uint32_t UEBinkAudioDecompressMemory(uint32_t rate, uint32_t chans)
{
    return BinkAudioDecompressMemory(rate, chans, 0);
}

// open and initialize a decompression stream
static uint32_t UEBinkAudioDecompressOpen(void * BinkAudioDecoderMemory, uint32_t rate, uint32_t chans, bool interleave_output, bool is_bink_audio_2)
{
    uint32_t flags = 0;
    if (is_bink_audio_2)
        flags |= BINKAC20;
    if (interleave_output == false)
        flags |= BINKACNODEINTERLACE;
    return BinkAudioDecompressOpen(BinkAudioDecoderMemory, rate, chans, flags);
}

//do the decompression - supports linear or ringbuffers (outptr==outstart on non-ring), will clamp, if no room
static uint32_t UEBinkAudioDecompress(void * BinkAudioDecoderMemory, uint8_t* OutputBuffer, uint32_t OutputBufferLen, uint8_t const** InputBuffer, uint8_t const* InputBufferEnd)
{
    BINKAC_OUT_RINGBUF out;
    BINKAC_IN in;

    out.outptr = (char*)OutputBuffer;
    out.outstart = (char*)OutputBuffer;
    out.outend = (char*)OutputBuffer + OutputBufferLen;
    out.outlen = OutputBufferLen;
    out.eatfirst = 0;

    in.inptr = (char*)*InputBuffer;
    in.inend = InputBufferEnd;

    BinkAudioDecompress(
        BinkAudioDecoderMemory, 
        &out,
        &in );
    
    *InputBuffer = (uint8_t const*)in.inptr;
    return out.decoded_bytes;
}

// resets the start flag to prevent blending in the last decoded frame.
static void UEBinkAudioDecompressResetStartFrame(void * BinkAudioDecoderMemory)
{
    BinkAudioDecompressResetStartFrame(BinkAudioDecoderMemory);
}

static UEBinkAudioDecodeInterface Interface =
{
    UEBinkAudioDecompressMemory,
    UEBinkAudioDecompressOpen,
    UEBinkAudioDecompress,
    UEBinkAudioDecompressResetStartFrame
};

UEBinkAudioDecodeInterface* UnrealBinkAudioDecodeInterface()
{
    return &Interface;
}
