// Copyright Epic Games, Inc. All Rights Reserved.
#include "binka_ue_decode.h" 
#include <stdio.h>
#include <string.h>
#include <stdlib.h>

#include "console.h"
#include "binka_ue_decode_test_data_in.h"
#include "binka_ue_decode_test_data_out.h"

#ifdef TEST_READ_CHAN_DATA
RADDEFFUNC bool RADLINK BinkAudioDecompressTestReadChanData();
#endif

CONSOLE_MAIN
{
    console_init();
    bool Failed = false;

    #ifdef TEST_READ_CHAN_DATA
    Failed |= BinkAudioDecompressTestReadChanData();
    #endif

    UEBinkAudioDecodeInterface* Interface = UnrealBinkAudioDecodeInterface();

    void* DecoderMem = malloc(Interface->MemoryFn(48000, 1));

    char* InputBufferBase = (char*)malloc(BinkBlocksSize + BINK_UE_DECODER_END_INPUT_SPACE);
    char const* InputBuffer = InputBufferBase;
    memcpy(InputBufferBase, BinkBlocksData, BinkBlocksSize);
    char* InputBufferEnd = InputBufferBase + BinkBlocksSize;

    short* CheckBuffer = (short*)PcmBlocksData;

    Interface->OpenFn(DecoderMem, 48000, 1, false, false);

    
    int BlockIndex = 0;
    for (; InputBuffer < InputBufferEnd; BlockIndex++)
    {
        // bink audio requires all buffers be 16 byte aligned.
#if defined(__clang__)
        short __attribute__ ((aligned (16))) OutputData[1920];
#else
        short __declspec(align(16))  OutputData[1920];
#endif

        U16 BlockSize = *(U16*)InputBuffer;
        InputBuffer += 2;

        char const* Input = InputBuffer;

        U32 Generated = Interface->DecodeFn(DecoderMem, (uint8_t*)OutputData, 1920*sizeof(short), (uint8_t const**)&Input, (uint8_t*)Input + BlockSize);
        if (Generated != 1920*2)
        {
            console_printf("Block %d wrong output bytes: %d, wanted 3840.\n", BlockIndex, Generated);
            Failed = true;
            break;
        }

        U32 Consumed = (U32)(Input - InputBuffer);
        if (Consumed != BlockSize)
        {
            console_printf("Block %d wrong input bytes: %d, wanted %d.\n", BlockIndex, Consumed, BlockSize);
            Failed = true;
            break;
        }

        for (U32 i=0; i<Generated/2; i++)
        {
            if (abs(OutputData[i] - CheckBuffer[i]) > 1) // we allow 1 due to float differences 
            {
                console_printf("Diff at Block %d / Sample %d: %d\n", BlockIndex, i, OutputData[i] - CheckBuffer[i]);
                Failed = true;
            }
        }

        CheckBuffer += Generated/2;
        InputBuffer = Input;
    }

    free(DecoderMem);
    free(InputBufferBase);

    console_handleevents();

    if (Failed)
        console_printf("Failed.\n");
    else
        console_printf("Succeeded.\n");

    return Failed ? 1 : 0;
}

