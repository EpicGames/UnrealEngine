// Copyright Epic Games, Inc. All Rights Reserved.
// Intel endian format

#define BinkBlocksSize 6940

unsigned char static const BinkBlocksData[BinkBlocksSize]={
 132,  1,168,126,160,161, 35,220,229,165,130,161,  1,177,160,128,112,176,176
,160,176,160,128,160,112,112, 80,112, 96, 80,128,240, 16,146, 69,  0,104
, 94,232,166,  5,154,229,114, 75,  4, 74,186, 33,204, 73,125, 96,216,211
, 33,133, 41,205, 70,189,100, 78,160, 57, 96,222,150, 75,168,208,100,136
,211, 76, 41, 52,102,144,200,145,  8,  1,145,132,112, 80,152,  2, 68,  0
,168,242,104,  2, 65, 76, 49, 66, 52,154,101,176,162,155, 44, 65, 26,191
, 75,178,  4,176, 73,101, 86,201, 12, 82,158,164, 36,196, 10,  6,193,171
,157, 75,158, 40,254,202,196,204,114,  3,161, 49,201,156,187, 83,194, 16
, 34,137, 80,213, 32,239,211, 42, 21,  1, 48, 33, 16,205,100,226, 98,180
,157,  9, 68, 82,105,100,  0, 42,156, 64,102, 46,101,164, 74, 14, 40,154
, 53, 48, 84,228,169,160,137, 39,225,116, 88,139, 89,200,108, 67, 49,146
, 43, 75,120, 53,156, 48, 51, 67,171,146, 10, 64, 73,132, 48, 56,198,  8
,116,  8,  0,113,117,244,155, 28,  8,  8, 66, 38,240, 33, 76, 46,  4,224
, 41,196,  2,208, 33, 46,  4, 25, 35,123, 72,199,128,161,  5, 32,216, 33
, 65, 83,225,216,  3,240,185,193,  0,  0,192, 67, 16,192, 48, 96, 81, 96
,  9, 16,184, 24, 24,  2, 22,208, 37,128,140, 22, 70,  1,129,162,128,  5
, 64, 74,124,  0, 49,  6, 12,  0,128, 18,176, 24, 12,128, 30, 32,  8,  0
,131,  3,243, 24,  3,128, 21,  4, 64, 65,112,  2, 10,192,130,  0, 12, 24
,  3,  0,232,209,  2,160,  0, 22,139,  6, 64, 14,  0,  0, 98,236,131, 12
,128, 37,  8,154,  0,112,  0,  0,  3,200,  7, 36,  0,  0,  0,  1,216,  8
,  3, 88,  0, 28, 13, 24, 96,  8, 16,  1, 64, 64,128,  0, 67,  0, 24, 48
,  0, 24,  0,  0,  8,  7,128, 15, 31, 58,  0,  0,  2, 44,202,178,110, 86
,119,137,173, 34,  2, 67,162, 34, 98,194,209, 17,178, 81, 49,209,208,208
,208,160,160, 80, 96,128,240, 16,146, 21,  8,130, 28, 66, 18,100, 17, 36
,173, 34, 25,181, 58,100, 41,  8,128,134,192,136, 46, 35, 57,  4, 65, 10
,197,  9, 16, 11,  9,170,178,144,168,167, 41, 44,162,148,249,117, 11, 29
, 27,101,149,205, 69,152, 24,112, 49, 34, 65, 42, 67, 44,147,178,212,164
, 33,238,238,130, 52, 86, 33, 86, 99, 89, 97, 84, 84,220,252,124,144, 41
, 59,228,187,188, 43,204, 82, 87,147, 14, 19, 19,203, 10,177, 44,199, 66
,182, 92,100,207, 45,166,237,139,233, 15,209, 17,161, 30,108,204,138,146
, 14, 95, 32, 43,168,144,228, 42,126, 42,136, 72,137, 89,128, 90,176, 68
, 39,151,117, 70,103, 71,133, 69, 33, 90, 62,232, 81,232,236,104,121, 49
, 49,166, 71, 35, 18, 67, 74,242, 52, 37, 85,133,101,125, 38,144, 36, 83
,192, 52, 24,209, 16,164,  8,209,232, 72, 64,220, 80, 70,118,  0,128, 12
, 34,156, 34, 19, 90, 57, 99, 90,160,129,192, 98,148, 51,164,169,203, 13
, 33,220, 61,131,132,240,240,144,182,229,241,153, 80, 64,173,107, 17,178
,180, 25, 75,136, 88, 89,123,208, 94,236,132, 14,147, 35, 82,  5, 88,198
,143,209,208,242,218, 12, 68,100,145,232, 34,184,234,  2,  0, 16,195,214
,179,  0,  2, 13, 73,  0,128, 34, 16, 48, 38, 80, 85,102,  9, 75, 80,250
,227,  1, 64, 14, 12,215,172,  2,224,129,160, 97, 44,  2,  9,163, 76, 31
,  3, 16,  0,192, 41,  8,144,234,  1,  6, 48, 22,  0,  1,  8,  1,130, 42
,227,  4,160,  1, 84,202,  5,200, 68,134,  1, 64,127, 10,114,232, 18,131
,236,  0,159,113,219, 33, 86, 10,132,199,192, 68,140,  9,208, 11, 37,244
,132, 97, 44,122,202,225,181,  0, 80,196,140,131, 96, 18,  6, 64,  6, 33
, 51, 43, 12,  0,208, 14,118,105,161,  7,  5,146,216, 89, 33, 24,160,129
, 77, 29,226,128,129, 18,112,  0,129,  3, 16,134, 48, 11,  0,  4,  0, 10
, 37,192,  7, 24, 81,128, 33, 28,  4,  0, 56,  0,102,101,176,  2, 64,107
,  1, 30, 96,  9,  0,  0,  0,160, 29,236, 16, 42,  0,  0,192,114,  1,116
, 16, 66,  3, 16, 64,192,  0,  2, 67, 24,133,232,176,  0, 58, 76, 43, 26
,  4, 16,  0, 32,128,  3, 65,  0,224,  2,  6,  0,  8,  0,156, 15, 31, 10
,  0,  0,  0,  0,  2,176,253, 49,173, 22,151,245,174, 50, 82,147,162, 83
, 35,131, 50,227,194, 98, 18,226,225,225,209,177,145, 97,161,113,177, 17
,146,133,192, 10, 34,  4,251,161,  8,245, 32,167, 22,140,134,  6, 11, 97
,192,142,136, 87, 30,172,161,111, 20,  8, 72, 82,189, 82,211, 98, 15, 32
,136,193, 56,140,133,105,140,199, 28, 40,170,141,110,225,207,166, 73,108
,157, 67,  9, 99, 82,  7,197, 89, 36, 57,  0, 13,148, 84, 95,171,239,179
, 46,203,178, 10, 52, 77,227, 89,228,251, 30, 42,116,119,228,243,174, 79
,124,238,150, 56,149, 99, 89, 78, 64,223,119, 89,150,229, 57,193,250, 46
,203,178,154,170, 66,158,245, 93,158,209,208, 97,201, 32, 16, 26, 20,134
, 35,100,129, 84,218,217, 81, 33, 86,227, 84, 38, 67,133,101,133, 92, 13
,105,194, 36, 50, 16, 75, 77, 43, 42, 42, 65, 26,175,113, 12, 56,200,104
,113,185,168, 16,218,116,159, 97,218,200, 72, 33,150,213, 52,226,116, 34
, 77,169,172, 16,154,208, 93,155,  4,144,180, 34,144,  8,179,214,195,197
,179,163, 16,150, 36,136,118,198, 64, 18, 74, 76, 71,240,142, 26,167, 77
, 28,171,  4, 48,200,162, 13,242, 46,163,162,145,193, 19,164,122,145, 99
, 85, 86,101, 82, 54, 68, 88,130, 52,205, 83, 93, 95,117, 76, 12,207, 10
, 82, 44, 83, 24,165, 21,149, 12, 37,158,  4, 36, 73, 33, 80, 53, 77,229
, 64,213, 84, 96, 21,176, 28, 74,160,179,172, 16,119,156,105,145,198, 42
,109,152, 72, 96, 88,116, 50,105, 67,135, 52,205,119, 76,168, 44,241,  4
,154,131, 22,123,178,  9,195,146,195,213, 19, 75, 10,164, 12,177,  8, 22
, 77,147,238,137,211,192, 24, 13, 21, 29,143,209, 89,145,151, 64,248, 45
,146,120, 38,105, 13, 45,144,179, 65, 98, 57,114,196, 77, 77,192,113,176
,145, 12,174, 13, 44,169,172, 52,173,218,208, 64, 45, 28, 93,173,112, 74
, 35,147, 84, 56, 20, 89,215,213,198,106, 66,  8,104, 33,210,120,159,101
,160,178,213,180, 65,212,208,132, 54,144, 74,160,153,224, 12,188, 48, 25
, 27, 73,  9,201,217, 92, 66,144, 92, 49,111,120, 16,132,184,238,  8,109
,136, 73,219,  4, 55, 38,116,214,151, 26,104,128,124, 64, 90, 61,114,202
,236, 36,146,120,162, 11, 57,172,133,149, 29,236, 72,219, 50, 41, 17,102
, 79, 83, 48,  0,157, 91, 55, 70,192,  1, 62,124,  0,192,  2, 44,197,113
,163, 54, 54,207,164, 66, 50,179,130, 51,179,115, 83, 67, 99, 51,147, 82
, 50, 98,194,  1,162, 49, 81, 97,113, 17,146,133,  0,200, 44,210, 73, 21
,137, 84,176, 29,196,192, 24, 19, 11,152, 24,228,  6,149, 61,210, 38,125
,210, 43,154,178,218,221, 61, 83, 88,142,173,202, 88,  0,178,  6, 18, 66
,141,171, 27, 74,227,172,205,205,105,136, 67, 40,132,129,132,208, 36,131
, 24,194,179, 68,198,216, 14,147, 98,148,114, 27,179,195,146, 65, 26,183
,153,204, 42,133,155,102,144,125, 86, 83, 76,210,198, 68,151, 38,117,  5
,185,178,169,215,240,168, 76,168, 49,105, 46,121,182, 87, 51,155,164, 73
, 43, 40,200, 20, 75,133,153,198,208, 81,185,165, 57,140,120,227,  2, 82
,168,105,130,101, 33,162,240,210,226, 68,165, 91, 56, 10,103, 35, 57, 69
, 28,147,180,164,  1,104, 34, 27, 90,103,132,206, 65,161, 99, 73,  9,148
,162, 59, 39,220,  0,104,200, 42,129,225,  0,  9,  7,117,  6, 49,180,177
,105,169,133,172, 21, 28, 66,156,101,129, 56, 39, 71,155, 18,158,129,166
,174,236,177, 19,221, 49,151, 72, 73, 74, 68,230, 48,173,249,132,155,218
,160,203,246,145, 52,  8, 42,238,172,204,172, 18,163,146, 64,164,  1,199
,196,155,  7,  2, 33, 10,  9, 22,133,152,104,  8, 66,140, 17, 88,124,229
, 74,231, 16,143,151, 79, 76, 89, 30, 18,177, 20,137,230, 56,150,208, 68
,102,  1, 18,  8, 47,118,137, 96,169, 34, 66, 76, 37, 38, 25,131,152,205
, 64,176,156, 58,149,  1, 58,201, 34, 72,116, 46, 38,241, 14,130, 34,140
, 49,155, 62, 38,138, 12,179, 70,183, 71, 11,212, 28,217, 89,119,132, 35
, 91,198,148, 38, 27, 59, 15, 25, 37, 52,164,  8, 32,128,169,116, 40,  3
, 36,141,212, 48,193, 32,190, 36, 85,250, 97,112,160,160, 10, 41,116, 24
, 18, 80, 56, 68,128, 73,137,108,159, 65,133,201, 90,235,204,134,167,  6
, 85, 90, 74, 16, 65,  8,198, 40, 45,130,233, 68,121,225, 80,130, 19, 42
,146,225, 65,211,192,137,154,196, 48,249, 63,146,220,131, 22,215, 70,109
,154,106, 78, 20, 25, 45,107,194,  6,167, 75,141,156,  8, 53,231,160, 58
, 54,186,  0,  6,199, 51,239, 45,117,170,108, 13,224,105, 12, 83,201,125
,129,204,218, 25,162,172, 48,104,144,  8,100, 24,114, 37, 33,109,149,176
,104,161, 65,199,142,169,234,132, 32,196,196, 64,162, 20, 49, 34, 41,137
, 81, 66,119, 76,160, 49,196,160,101,228, 48,108,105,154, 80,118,206,236
,136, 68, 99,144,129,131,153,  1,130, 56, 40,130,  8, 66, 10,  4, 33, 79
,145,100,172, 18, 41,115, 12, 66, 80,137, 50, 82, 86,129, 80,149, 83,196
, 96,198,168,139,181, 71, 12,169, 38,114, 60,135, 58, 89,  3,153,187,107
, 56, 34, 68, 60,107,204,209, 40,178, 32, 10, 20, 42, 84,100,101, 89, 97
,124,111,172,122, 51, 75, 21, 25, 94,169, 74, 26, 96, 16,200,194,217, 24
, 57,192,144,151, 84,214,109,213,194,198, 98, 86, 52, 30, 84,136,192,161
,102, 11, 26, 70,194,221,223,165,150,133,  0, 65,199, 12, 34,129, 13,163
, 66, 52,149, 42,230, 90, 16,194, 72, 72,192,148, 20, 50,235, 14,142,251
,  6,131, 32, 63, 16, 55, 24, 16, 98, 64,128, 15, 31,  0,  0,  0,  0,164
,  2, 48, 21,190,242,101,196,137,175,114, 98, 83, 83,146,131,163, 19,148
,115,211,114,130, 98,130, 98, 66,242,145,177,145,113, 17,146, 21,  8, 34
,  6, 28,186,248,194,195,  7,  9, 74,  4,163,152,186,113, 19,168,249, 68
, 69,  4,136,165,138,254,250,  8,  5,128, 72,170,176,154, 44,109,249,200
,139, 58,152, 14, 78,222,110,165,166,108, 74, 46, 16,130,252,100,163,154
,  9, 36,150,211, 52, 76,160,130, 16, 26, 49, 11, 74, 69,230,170,106, 66
, 25, 97, 40, 82, 17,145,136,128, 17,  6,161,120, 68, 16, 82,134, 97,206
,156, 16, 29, 49, 21,216, 34, 18,  6, 29, 88, 90,195, 41, 69, 80, 38,211
,158,236,101, 70,176,195,153, 27, 19,181,179, 21,196, 35,164, 52,178,209
,156, 40,  8, 20, 71,149,193,216,168,146,  9,163, 33,108, 14, 84, 14,115
, 48,246,148, 41,132, 50, 76, 84,187,149,154,167, 69, 11,233,177,220, 35
, 50,161, 50,229, 97,  5,164, 97,137, 90,  1, 88,  4,136,142,200, 89,165
,156, 20, 90,232,232, 12, 86, 34,104,214,116,200,227, 80,168,199,174, 56
,130, 12, 81, 13, 83,165,129,104,179, 25,101, 20,132, 79,164, 16,  4,142
, 54,194,  8,211, 25,156,161, 34, 59, 11,253, 67, 52, 96, 14,183,104,179
, 94, 89,219,131,  0,135, 24,  5, 45, 99, 92,107,202,173,103,182,147,150
,124,100, 99,116,200,177,211,218, 36, 97, 16, 42,228, 93, 18, 43,204, 96
, 32, 32,178,  2,130,  5,136,  8, 20,195, 20,144,102,187, 52, 34,166, 73
, 25, 58,136,177, 64,168,169,160,152,246,150,162, 39, 70,195,114,121,204
, 12,210, 58, 16, 25,104, 80, 85, 21,141, 16,194,148,164,150,109, 46,202
, 65,169,104, 10, 25,196,200,198, 52,181, 80,134,  4,146, 65, 14,138,116
,138, 33, 14, 42,240, 25,227, 33,172, 59,192,104,216,130,107,107,137,  1
,210,175, 56,152,138,136, 80,  9,162, 61, 93,119, 72,210, 48,168, 18, 17
,  6, 49, 44,202, 34, 32, 24,106,107,  6,209,132, 40, 36,  8, 59, 32, 98
, 82,198,240,150,108, 20,132, 70, 98, 34, 18,195,188,148,  8,  3,149, 53
,198,104,162,100,115,197,  3, 13,110,109, 73,177,167,132, 54, 21, 33,172
,113,  9,  4,107, 44, 64, 96, 26,229,  7, 38,114, 50, 73,  2,134,146, 37
,181,100, 69, 78,106, 20, 34, 55,216, 56, 86, 51, 55, 16, 69, 51, 78,  1
, 16,134,161,172,147,182, 42, 52, 27, 17, 40,130,201, 70,  6, 58,232,145
,200,142, 26,102, 32, 91, 59, 74,124, 94,170,135, 98,186, 97, 12,  1,132
,142,227,156, 65,250,171,121,125,128,146, 39,171,235, 83,133,193,144,132
,118, 45,247,116, 53,193,122,205,140,  2, 20,170, 25, 40, 96,160, 49,  4
, 58,113,180,193, 90,105, 60,167,213,117, 16,208, 72,194,136,216,180, 38
,211,157, 12, 10,234, 88,100, 71, 10,235, 72, 63,237, 65,168,160, 81,101
, 48,216,176, 67, 16,  2,  4,  0, 37,193, 24,  1,122, 49, 73, 35,128, 40
,187, 66,  1,128, 64, 80,130,136,254,131, 36, 65, 73,  1,212, 80, 34,  1
, 48, 50,240,130,144, 82, 68,  9,116,120, 18,195, 26, 16, 32,196, 25,129
,  0, 72,128,  0,194,192,135, 15,  0,  0,  0,152,  2, 44,220,107,246,230
,150,100,165,162,210,114,179,210,114,  3, 51,116,147,147, 34, 34,  2,114
, 98,114,210,161,177, 97, 97, 17,146, 53,  8,142, 61, 24, 97,148, 83, 78
, 58, 93,115, 15, 62, 26, 77, 81,170, 28,194,140, 56,143,177, 42,153, 16
,  9, 10,225, 40,138,195, 64, 18,105,161, 10,228, 80, 10,146,145,163, 19
, 41,  1, 57,108,  6,178,153,230,155,111,190, 26,227, 24, 99, 12,147,136
,169,209,148, 44, 76, 72, 16, 69, 81,148, 51, 18,129, 92,246, 86,145,204
, 72,  0,  0,203,134,198, 98,147,179, 53,203, 76, 17,133, 83, 81, 29, 51
, 12, 41,177,108,112, 52, 54,150,  2,161, 65, 40, 17, 38,146,117,133,144
,102, 64, 65, 82, 83,113, 24, 14, 21, 71, 84, 74,  8,  2,195,209,224, 88
,201, 89,116, 50, 32, 32, 56,133, 56, 76,108, 36, 82,153,219,125,149, 14
,165, 18, 41, 16, 82,205,136,  6,249, 49, 33,222,229,142,241,101, 86, 72
, 36,175, 12, 47,129,  7, 75, 27, 65, 94,183,175, 38,149,140, 65, 53, 30
,121, 15,193,164,228,  5,163,132, 51, 43,191,114,148, 50,194,138,130, 42
, 33, 51, 52,160,146, 41, 74,144,  0, 84, 44, 11,129, 14,  6,217,145, 74
, 29, 39, 38, 81, 76,140,250,226, 57, 80, 17, 82,  4,129,195,248,242,167
,182, 35,102,140,140,136,164,214,241, 26, 10,193, 12, 73,  6, 97, 83, 59
, 89, 55, 48,152, 12, 41,  8,167, 70,235,174, 67, 18,  5, 19,138, 80,164
,228, 46,234, 72,144, 72, 29, 17,  8, 68, 25,240,189,148,  6, 66, 68,196
,  0, 35,  4, 13, 49,224, 28,231,149, 49, 14,117,134, 16,119,210,128, 84
, 65,138,231,128, 72, 97,161, 72,152,154,  9, 13, 25,240,166,117, 26, 52
, 39,214,  0, 50,246, 44,  8,189,216,236, 50,153, 16,145,132,140,109, 76
, 12, 38,130,128,161,178, 33,145,  8,136,128, 73, 28,146, 56, 12, 71, 81
, 67, 37,250, 91,123, 35, 80,  0, 81,  4, 20, 12, 32, 70,184,187, 26, 41
,146,137, 51, 20,  9, 18, 66, 20, 48,204,147, 99,180,164,179,236,197, 80
, 88, 87, 68, 22,154,181, 78, 56,104,192,229,  2,234,180,122, 78, 25,112
,181,232, 10,121,132,164, 97,192,189, 19,145, 65, 69, 17,213,104,194, 16
,115,170, 17, 70, 62,105, 52, 80,  1,229,158, 67,201, 12, 83,164,136,134
, 40,  3, 43, 28,136, 10,  1, 24,  3, 64,168, 90,103, 41,195,180,182,106
,231, 36,149,202, 36,  5, 74,167,128, 16,200,156,  4, 80,  8,  9,105, 98
, 82, 42, 53, 73, 57,147,149,163,123,209,138, 78,221,200, 85,229, 88,169
,248,141,210,  0, 21,153, 70,212, 24,153,137, 56, 18,  4, 64, 72,192, 82
, 24,210,125,214,  1,  2, 48,  1,  6, 42,102,165,162, 59,104, 81, 82,229
,222, 41, 76,177,187, 70, 16, 83, 74, 56, 42,202, 83,136,245, 55, 42,151
,134,172,178,144, 25,169, 24,  9,162,160, 19,234,160,192, 20,208, 91,103
, 12, 99, 38,121,  1,  0, 57,138,  2,195,177,155, 17, 33, 96,241,128,132
, 66,226, 17, 10, 80, 21,126, 33,  5,112, 73, 17,120, 84,  1,129,129,195
, 11,  8,  8, 36, 16,  5,  4,152, 15, 31,  0, 56,  2, 44, 17,  0,174, 86
, 73, 16,172, 98,162, 66,163,146, 66,  3, 66, 19,147,242,241,145,129,161
,241,  1,242,113,145, 97, 65, 17,146, 21,200, 45,  3,163,168,164, 78, 98
,107,153,163, 14,196, 64, 30, 10,132,232,172,182, 68,197, 88,225, 46,154
,166,234,137, 34,124,105,214, 21,106, 40,120,  5,129, 88,138,196,242, 52
,203, 65,116, 71, 32,117, 80, 27,164,206,121,255,  7,136, 49, 76,102, 38
, 98,138,198,136, 34, 67, 40, 34, 13,128, 72,  4, 97, 19, 65, 35, 18, 17
,162, 65, 33,  1,162,212, 33, 65,  6, 17,114,123, 17, 18,100,219, 32, 73
, 64,130, 48, 68, 51, 64, 77,156,213,136, 38,153, 47, 80,198, 40, 82,105
, 20, 49,186, 26,148, 97,103, 40,212,144, 52, 93,  6,201,154, 16,151,240
, 27,222, 93,133, 25, 25, 85,182,152, 76, 72, 17,146, 52,146,162,134,105
,127, 86,164, 46, 51,175,170, 18,187,212,181,209, 68, 76, 21, 21, 17, 73
,168,237,191,133,182,190, 11,216,110, 84, 70,100,136,  6,154, 25,100,119
,178,172, 50, 35, 34,161,186,173, 49,  4, 55,193,213,106,  0,119,  3, 19
,  8,224,136, 28,213,200, 51,157,138,  7, 50,248,164,137, 93, 17, 85,153
,  0, 16,161, 76,152,236, 76,194,229, 17, 34,140, 52, 38, 89,103,176,156
, 51,185, 58,  6, 85, 33,217, 37, 25, 69, 90,154,129, 38, 17,120,201,130
, 89, 42, 22, 67, 37,169, 65,130, 94,162,108, 16,129, 65, 20,156,171, 40
, 70, 89,  3, 65,195,228, 12,105,202, 16,149,102,138, 81,140,168,103, 18
,104,179,218, 64,104,178,113, 72, 88, 12,169,200,106,116, 74, 61, 40,131
,184, 83,105,155,224, 19,163,190, 35,170, 32,  9, 68,169, 81,229, 69, 40
,147, 87,253,205,152, 84, 33, 73,101, 45, 66,106,132,114, 38,176, 75, 56
,212, 36, 15, 40, 19,162, 45, 67,169,211,161,145, 42, 60, 11,236,116, 40
, 85,146,139, 38,216,112,170,147,227, 17, 17, 73, 43, 21,146,101,  5,221
, 94,112, 16, 50, 35, 82,179,  4,169, 34,150,160, 33, 13,  0, 75, 43,153
,201,178,130, 76,181,200,205, 32,107, 75, 51,221,153,240, 25, 68,168,166
, 25, 86,139,212,151, 34, 95,106, 87,168,149, 98,127, 74, 91,111,130,152
, 28, 93, 38, 20, 21,172, 32,137,195,159, 52,177,138, 72,199,  4,171,140
,180,  2, 72, 69,133, 34,210, 85,100, 19,224,172,  1,207,104,194, 40,  2
, 66,156, 67, 52, 63,  6,160, 79,106, 38, 20, 12,235,144,174,114, 16, 51
,154, 26,219,104, 59, 67, 23,202, 11, 12,  3, 51, 10,161, 93, 32,  1,213
,130,  0,  2, 48,  0,132,152, 32,192,128, 16, 65, 16,  0,  2, 33, 32, 16
,124,248,  0,  0,  0,184,  1, 48,141,  9, 37,183,104,182,164,130, 17,209
, 49, 81,225, 49, 97,178,209,  1,209,160,144,176,144,176,160,128,144,128
,240, 16,146, 21,  4,  2,  8,157,198, 59, 26, 66,213,144, 10, 83, 20, 89
,188,225,  0,135,168, 43, 48, 87,  6,175,214,230, 40, 36, 70, 25,145,188
, 34, 76,153,179, 34,241,120, 12,129,  0,  5,106, 84, 16, 65, 70,224,215
,223, 24, 65,134, 73,104, 70, 33,145,194,155,185, 36,145, 64, 50, 71, 85
, 33, 18,100, 16, 45, 49, 17,136,154, 41,242, 19, 18,194, 36, 98,144, 67
, 22,130,200,153, 32,155,150, 40, 73,148,165, 97, 36, 99,148,  0, 17,154
, 68, 76, 16,105, 17, 89, 58,133, 44, 12, 34, 36,198,104, 44, 37, 66,164
,200,162, 72,117,131,218, 96,220, 84, 19,109,237, 26,164,162, 36,152, 68
,  5,174, 48,163, 44,200, 45,139,100,104,204,144,161,  8, 14,107,120, 85
,141, 22,145,244,147,  6, 32, 10, 19,  5, 76, 64,178,197,102, 71,146,108
,204,180,149, 11, 81, 10,118,139, 26,214, 90,111, 82,172,210,188,103, 52
, 19,153, 32, 13, 29, 81,150, 36,129,134, 14, 85,144,204, 16, 72, 34,  5
, 81,132, 86,132, 52,167, 59,198,116,164, 66,228, 43,131,  0, 97, 32, 20
, 16, 96, 37,160, 64, 48, 16,139, 77,112, 16,131,138,101, 60,128, 48,  8
, 17, 49, 12,  2, 36, 34, 89,  4, 32,218, 48,  0,120,112, 32,118, 97, 24
,168,242,  2,  8,  2,192,180, 40,  8, 49, 70, 72, 84, 68, 32,  9, 77,102
, 71, 47,171, 67,131,128, 42, 96, 76, 66, 24, 72, 41, 32, 56, 34,137,  1
,  2,102, 14,  0,248,  0,182,141,  0, 86, 70, 32, 35, 67, 50, 24,154, 25
,204, 16,102, 76, 64,192, 12,129, 96,  3, 96,  4,188,  0, 64, 48,128,102
,  4, 80, 96, 46,136, 54,  0,  1,152, 99,  0,180, 98, 65,  0,224,152, 85
,134,  2,  8,  2,112,  0,  4,176, 49, 82, 67,  0,  0,  2,  0, 96,  1,  2
,128, 32,165, 41,  6,  8, 24,129,128, 16,  4, 10,  8,  2,  0,184,  0,117
,  5,  3, 64,  0,  1, 12, 32,  0, 96, 62,124,168,  0,  0,  0, 40,  2,180
, 40,242,243,246, 97,114,182,178,226,226,146,210, 98,226,129,146, 34,211
,130, 34, 18,194,161,193, 33,178,193,241,225, 17,146,165, 98,  4,  4,232
, 66,130, 68, 85,109,197,157,102,189,113,133, 73,160,136,183,158,176,202
,173, 30,141,100,249,234,150,171,178,177,231,178,238,182,128, 80,135, 74
, 12,129, 89,133, 35, 42,157,194, 72,  4, 99,166,171, 82, 37, 84,100,147
, 82,  9, 58,169, 33, 67, 92, 97, 86,147, 54, 49,234,199,  9, 67,140,209
, 46,105, 17, 69,132, 22, 73,168, 74,242,204, 86,206,236,200, 29, 70,117
, 24, 18, 78,138,  5,229, 78,104,196, 16, 44, 28, 26,185,195,204, 21, 70
, 99,162, 71, 88, 50,103, 49,170,164, 58,216,210,  8,114,162, 39,140, 81
, 73,  0,195,152, 92, 90,201,140, 96, 83,  5, 81,  1,114, 51, 72, 80, 75
,  6,145, 33,165,226,195, 34, 12, 50,205,147, 50,115,114,133,124, 72,196
,165, 51,190,152,181,209, 81,141, 74,164,210, 70, 16,244,132,162, 90,113
, 48,194, 35, 28,147, 69,172,  2,208, 52,157, 19,  7, 32,149,213, 52, 38
, 52, 98, 99,154,100,239,106,194, 80,131, 88,144,195, 19, 26, 64, 16,170
, 18,160, 41, 69, 97, 86, 98,149,228,135, 56, 71, 67,131,168,250,210, 10
,147,226, 49,198,136, 72,149, 80,132,112, 76,174,136, 38, 37,177, 82, 72
,  5,172,155,  0, 52,113, 72,227,213,107, 85, 25,213,195,  8,147, 54, 57
,106,  9, 13, 49,106,145, 61,142,122,233, 66,163,131,128,140,168, 42,135
,116, 76, 78,114,105,102,237, 22,156,164,101, 84,141, 65, 20, 71, 39,  6
, 27,210,  2, 52,169, 10, 41,  0,193,162,200, 16, 35, 19,138,234,225,232
,  1, 42, 35,147, 66,213, 38,237, 76, 40,105,  4, 26,179, 73,186,132,141
,209, 45, 48, 65, 17,161, 73,170,  3,205,129,223,102, 12, 99, 54,141, 64
,179, 24, 68,142, 74,133, 56,135, 33, 86,155,148, 20, 57, 64, 85,207, 49
,137, 80,  8, 15,193,123,116, 25,149, 76, 73, 54,242,238, 42, 42,195,164
, 41, 88,  9, 48, 69, 83, 44, 34, 52,105,149, 50, 35,144, 34,112, 65,248
,120,100, 15, 71, 45,138, 68, 86, 36,107, 41, 19,155,132,149,225,164, 13
, 68, 34, 98, 69, 56,136,179, 84, 39,135, 20,224,107, 85, 74,218,153, 88
,136,179,102, 50,144, 73,139, 34,138, 43, 28,153,238, 21, 37,180, 51,168
, 13, 33,214,206,184, 17,128, 38, 46, 17,112, 96, 25,103,205, 68, 52, 52
, 17,  3,138,208,164, 21, 13, 86,156, 56,122,228,196,  0,128, 32,164,168
,200,  4, 40, 33,130, 53,224,195,  7,  0,  0,148,  2, 40, 97, 60, 61, 37
,  6,157,167,162,114,195, 66, 19, 83,195,  2,179,  2,  3, 83, 18,226,161
,193,  1,210, 33, 65,145,145, 17,146,133, 56, 88, 98,203, 37,161,212,131
, 77, 98,200, 52,218,115, 82, 20,194, 67,  2,177, 73,102,  1, 33,130, 76
,187,184, 38, 67,177, 49,186,138,104, 80, 18,105, 97, 10, 72, 77,161,192
,235, 24, 66,140, 80, 64,  0,  1, 32,210,136,118,196,160, 28,130,129, 52
,143,229,153, 13,249, 10, 78, 34,181,182, 49,185,211,181, 58,164, 79,114
,219, 68,140,154,155,162,137,170, 52,162, 74, 34,165, 60, 70, 64,178, 17
, 19, 74,152,228, 24,201, 92, 45, 55,228, 12,194, 99,144, 28,226,  9,195
,226,130, 12,209,  9,  5,165, 34, 33,  4, 38,186,140, 49, 91,224, 97,205
, 81,229,130,132, 82,146,138, 74, 33,  9,178, 92,161, 87, 60, 51, 11, 53
, 76, 12, 88,134,  9, 42, 14, 56,233, 38, 98, 36,122, 39,117, 81,178,209
, 10, 99,239,152, 36, 66, 99,115,109,145,151, 41,198,161,138, 12, 36, 99
, 64,115,204, 50,170, 40,120, 14, 24,146,196,192,218, 84, 37,169,172, 10
, 83,146,128,241, 14, 10,196,142,  9, 36,113, 20, 54,137, 39,211, 36,113
, 33, 34,148, 73,  5, 18,133,118, 79, 95, 67, 74,  4,179, 75, 86,239,230
, 21,105, 52, 42,  6, 73, 53,134,152,131, 20,178,135,160,187, 42, 57, 43
,136, 98,131,168, 88,109, 42, 97, 96, 84, 40,206,117,  0, 20,160,179,196
,185, 98,101,169,144, 80, 24, 24,199,129,176,160, 38,132, 65,207,  8, 16
,  3,148, 65, 68, 34, 23,163,193,145,219, 18,163,217, 25,232, 73,202, 32
,129, 78, 36,141,128,104,181,166,193,  2, 21,147,137, 32, 25,164,112,101
,142,172,100,  8, 83,  4,108, 59, 67,141, 16, 37, 16, 62, 72, 70, 19, 33
, 79, 50, 76, 92,145,140, 81,142,141, 40,  1,132,105,243,129,166,177,236
, 41,112, 40,134, 44,  9,  8,171, 98, 86, 28,205,115, 16,  2,109, 78,232
,165,103,184, 36,201, 67, 69,166,137, 96,204,100,  8,247,186, 99, 56,212
, 32, 91, 26,155, 44, 32,174,149, 20, 13,142,225,184, 13,179,243, 35, 33
,130, 20,128,148, 81, 24,179,105, 26, 41, 74,186,220,221,198, 40,164,115
,  5,  9, 46,  3, 84, 39, 60,176,137, 12, 19,162,142, 54, 21, 43,142,192
, 42,143,  7, 77,114,163, 29,126,170, 11,109,  5,145, 24,222, 67, 86,100
, 27,  8,  8, 99, 89,220,228,132, 21, 51,152, 41,130,152,131,  7, 17,230
,  0, 25,112, 41,215, 64,131, 20, 49,130,220,149, 33,201, 12,202,236, 41
,129,186, 66, 19, 12,  0,170, 66,170, 40,226, 54, 21, 10,123, 43, 85,206
, 87, 34,200, 47,  3, 74, 64, 20,  8, 77, 10, 35,200, 98,212, 52,100, 43
, 14,137, 92,228, 12,  3, 85,121,107,196, 37,156, 32,178, 80, 49,196,132
,115, 28,139, 20, 53,167,166,  3, 41,211, 12,  1, 97, 66,204,161,163, 59
,185, 34,151,208,172,  6,108,228, 40,151,235, 37,200, 33, 98,210, 28,  2
,138,147,214, 16,194, 16, 94,  0,  4,105, 33, 96,  2, 35,196,210,  0, 18
,  1, 66,  0, 19,227,195,  7,132,  2,176,182, 91,232,214, 87, 13,164,210
, 98,195, 18,243,242,178,242,226,  2,195, 34, 18, 50,178,129,161, 97,241
,240,112, 97, 17,146,133,120,217,133, 27, 76, 74,169,133,107,204,138, 91
, 84, 32,  1,131,176,  8,200,112, 20,  8, 34, 78, 85,171,156,  2, 24, 68
, 52,160,196, 83,  2, 86, 92, 69,208,252,235,166,232, 86, 91,107,197, 83
,137,172,158,139,214,229,160, 84,198,  8, 33,128, 65, 64, 97, 55,130,237
,195,100,100, 46,213,111,171,102,102,162, 40, 35,  4,152,216, 44,102,174
, 70,130, 72, 74,146, 16,220,182,134, 87, 53, 21,138, 64,204, 92,227,  8
,100, 82,196, 24, 35, 26,172,101, 86, 52, 49, 33, 17, 24,205, 57, 35, 75
,221,182, 99,144, 96,145, 82,126,231,  4, 52, 34, 20,231, 25,164,137, 95
,156,164, 91,140,198,173,225, 56, 54,198, 80,140,136, 86,207, 70,153,200
, 38,144,233,204,129,166, 28, 42,  9,121,115,166,213, 94, 26,195, 64,147
, 46,215,119,129,  8,170, 64, 69, 68,114, 66,205, 51, 28, 84,198, 48, 97
,138,120, 17,225,156,119, 38,233,129,110,166,152,214, 80,208,  9,178,106
,157,195,112, 47,179, 54, 47,248,157, 17,129,226,180, 21, 87,169, 97,210
, 76,167,132, 50, 26,172,  9, 98, 34,205,226,228,204,212,201, 19, 77, 30
,233, 64,168, 33, 48, 72, 99, 93,243,249, 20, 77, 80, 10,  1,154,177,230
,138, 65, 26,168,168,104, 82,168, 17,164,225,  9, 55,207, 82,192, 33, 26
,160,  6,250, 83,211,112, 12, 90, 19,170,187,203, 18,193,220,171,153, 22
, 80,228,110,179, 10, 52,153,153,191, 59, 93, 67,177,251,169,139,196, 82
,235, 38, 86, 25,  9, 49, 98,165, 22,151,113,182, 18, 95, 22, 84,102,108
, 86, 16,160,  9,172,174, 88, 20, 42,  3, 45,136, 48,194,204, 28, 67,147
, 35, 91,255,157,145,230, 24,219,203,183,233,161,111,207, 64,114, 16,133
, 80, 16, 70,  4,203, 41,132,194, 56,201, 12, 50,140,197, 32, 64,139, 32
, 70,105,249,156,164,237,255,214,  0,185,203,171,132, 82,187,132,145,153
, 57, 27, 34, 73,196, 16,136,221,132,215,228, 51, 67, 34, 17,206, 23, 66
,  0, 17, 68,  8,  0, 27,130, 98, 72, 34,  3,169,228,135, 68, 24, 17,232
,178, 47, 10,227,164, 35,172, 54,221,220, 66, 52,232, 44, 38,176,207, 52
,140, 72, 64,134,132,102,192, 12, 11,148,144,  0,136, 53,111, 92, 39,140
, 49, 76, 50,123,123, 59,162, 16,104, 52,136,152,220, 84,104, 26, 66,103
,204, 24, 65,142,225, 13,137,220,  8,  1, 39, 50, 54,147, 53, 53,134,132
,222, 90, 90,236,191,109, 65,112, 38, 85, 51, 33, 52,  1,222,140, 84,200
,208,148, 64,217, 85,104,128,144, 72,254, 64, 24, 95, 41,142, 17, 83, 70
,106,241,157, 97, 66,138,136,102,155, 53, 87, 65,135,198, 24,  2,102,205
,119, 28,  1,  8,153, 72,109,197,219,206,146, 16,  1,121,121,223,149,164
, 35,192,148, 65,  4,155,  8, 96,152, 84, 32,150,120, 21,241,131, 80,128
, 15, 31,  0,  0,  0, 84,  2, 52, 70,138,108,166,105, 53,164,226, 82, 51
, 51,243,114,  2, 34,146,114,242,209,113,193,193, 49,  1, 65,241,144,144
,240, 16,146, 85,202, 37, 92,  2, 38, 84,129, 29,222,  1, 39,114, 98, 10
, 14,129, 16,134, 65, 24,  4, 34, 56,198, 11, 37, 30,106,144, 33, 26,209
,200,  8,144, 74,114, 59, 38,211,161, 68,162, 10, 26,104,225, 42,153,147
, 82, 48,  3, 51,204, 56,241, 44,237,160, 58, 84,150,200,105, 26, 76, 36
,  5, 55, 55,249,253, 26, 27, 36,124,135,130,  4, 58,227, 48,169,161,203
, 49, 20,  3,193, 36, 18, 82, 10,144, 42, 24, 94,100,164, 36, 73, 85,117
, 66, 76,198, 59,109,160, 97,246,149,209,212,219,160,145,  4, 81,105, 76
, 12, 11,179,235, 42,233,171, 65,144, 46, 27,203, 29,167,217, 30,  8, 70
,144, 44,179,155, 26, 38,193,122,222, 32, 33,128, 54, 24,129, 12, 28, 83
,  4, 18,101, 39,100, 77, 19,145, 26, 60, 42, 60, 27,129,149, 13,142,149
,157,149,204, 36,149,142,141,141,157, 15,197, 75, 32,114,237,190,122,102
,163,159, 44, 16, 70, 82, 91, 27,100,192, 99,192,194,177,  3,236, 54,  1
,122, 32,141,  4, 35, 24,192, 17, 53, 49,204,105,  9, 68, 19,157, 79, 12
, 37,170,214, 90,  8,  4,242,225,  8, 97, 50,116, 65, 16, 20,132,161, 99
, 13, 32,181,127,243, 48,204,112, 13,127,110,210, 64,  6,  2, 56,247, 76
,127, 20,231, 48, 14,  1, 10, 52,204, 53, 58, 57, 33, 75, 16, 77, 61,178
,203,235,168,115,160,171,170,235,137,136,211,212, 77,131, 17,229,150, 44
, 33,196,155,204,160,104,131, 72,112, 34, 12,208, 49, 75,192, 57,220,114
,205, 13,243,193, 77, 74,255, 36, 78, 93,142, 36,221, 48,117, 64, 97,136
, 20,214,239, 80,177, 21,165, 64,133,218,  9,  4,188,149,214, 68,  4,211
,144, 82, 25,120, 81,137, 64, 70, 65,210,  4, 64, 10, 14,116, 10, 76, 91
, 37, 14,147,142, 92,221,102, 81,  4,160,141,100,145, 16,101, 69, 64, 10
, 32, 32, 98,224, 10,196,212,  1,  0,124,220, 89,235,  5,209,  4,135, 82
,193,130,  2,116,139,161, 25,235, 81,164,130,133,192,156, 98, 96, 28, 36
,194,128,196,  1,101,218,233,  5, 64,248,  0, 43, 12,  1, 50, 51,  0,131
,177,232, 64, 19,166,  4,123, 55,165,112,103,  5, 29,166,183,127,118,123
, 24,  0,128, 32,  8,  2,136,190, 17, 97, 82, 32,194,194, 16, 65,192,118
, 16, 64, 77,195,146,171,  0,226,136,211, 82, 74, 13, 84,192,101, 14, 16
, 25, 70,170, 96,123,148, 52,  5,  4, 85,  0, 12, 21,  2,  0,182, 81,104
, 15, 20, 67,192,198,  0,  0,  8,176,144,  2,  4,  1,128,  1,192,152,  1
,  0,  2,150, 48,216,  0,140,193, 56,  0,  6,192,  0,  0,  0,102,143, 66
,  0,  0, 20,  1,124,248,  0,  0,  0
};