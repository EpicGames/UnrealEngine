// Copyright Epic Games, Inc. All Rights Reserved.
#include "rrCore.h"

#include "binkace.h"

#include "binka_ue_encode.h"
#include "binka_ue_file_header.h"

#include <stdio.h>
#include <string.h>

#define MAX_STREAMS 8

struct MemBufferEntry
{
    U32 bytecount;
    MemBufferEntry* next;
    char bytes[1];
};

struct MemBuffer
{
    void* (*memalloc)(uintptr_t bytes);
    void (*memfree)(void* ptr);
    MemBufferEntry* head;
    MemBufferEntry* tail;
    U32 total_bytes;
};

static unsigned int CRC32Table[] =
{
    0x00000000, 0x77073096, 0xEE0E612C, 0x990951BA, 0x076DC419,
    0x706AF48F, 0xE963A535, 0x9E6495A3, 0x0EDB8832, 0x79DCB8A4,
    0xE0D5E91E, 0x97D2D988, 0x09B64C2B, 0x7EB17CBD, 0xE7B82D07,
    0x90BF1D91, 0x1DB71064, 0x6AB020F2, 0xF3B97148, 0x84BE41DE,
    0x1ADAD47D, 0x6DDDE4EB, 0xF4D4B551, 0x83D385C7, 0x136C9856,
    0x646BA8C0, 0xFD62F97A, 0x8A65C9EC, 0x14015C4F, 0x63066CD9,
    0xFA0F3D63, 0x8D080DF5, 0x3B6E20C8, 0x4C69105E, 0xD56041E4,
    0xA2677172, 0x3C03E4D1, 0x4B04D447, 0xD20D85FD, 0xA50AB56B,
    0x35B5A8FA, 0x42B2986C, 0xDBBBC9D6, 0xACBCF940, 0x32D86CE3,
    0x45DF5C75, 0xDCD60DCF, 0xABD13D59, 0x26D930AC, 0x51DE003A,
    0xC8D75180, 0xBFD06116, 0x21B4F4B5, 0x56B3C423, 0xCFBA9599,
    0xB8BDA50F, 0x2802B89E, 0x5F058808, 0xC60CD9B2, 0xB10BE924,
    0x2F6F7C87, 0x58684C11, 0xC1611DAB, 0xB6662D3D, 0x76DC4190,
    0x01DB7106, 0x98D220BC, 0xEFD5102A, 0x71B18589, 0x06B6B51F,
    0x9FBFE4A5, 0xE8B8D433, 0x7807C9A2, 0x0F00F934, 0x9609A88E,
    0xE10E9818, 0x7F6A0DBB, 0x086D3D2D, 0x91646C97, 0xE6635C01,
    0x6B6B51F4, 0x1C6C6162, 0x856530D8, 0xF262004E, 0x6C0695ED,
    0x1B01A57B, 0x8208F4C1, 0xF50FC457, 0x65B0D9C6, 0x12B7E950,
    0x8BBEB8EA, 0xFCB9887C, 0x62DD1DDF, 0x15DA2D49, 0x8CD37CF3,
    0xFBD44C65, 0x4DB26158, 0x3AB551CE, 0xA3BC0074, 0xD4BB30E2,
    0x4ADFA541, 0x3DD895D7, 0xA4D1C46D, 0xD3D6F4FB, 0x4369E96A,
    0x346ED9FC, 0xAD678846, 0xDA60B8D0, 0x44042D73, 0x33031DE5,
    0xAA0A4C5F, 0xDD0D7CC9, 0x5005713C, 0x270241AA, 0xBE0B1010,
    0xC90C2086, 0x5768B525, 0x206F85B3, 0xB966D409, 0xCE61E49F,
    0x5EDEF90E, 0x29D9C998, 0xB0D09822, 0xC7D7A8B4, 0x59B33D17,
    0x2EB40D81, 0xB7BD5C3B, 0xC0BA6CAD, 0xEDB88320, 0x9ABFB3B6,
    0x03B6E20C, 0x74B1D29A, 0xEAD54739, 0x9DD277AF, 0x04DB2615,
    0x73DC1683, 0xE3630B12, 0x94643B84, 0x0D6D6A3E, 0x7A6A5AA8,
    0xE40ECF0B, 0x9309FF9D, 0x0A00AE27, 0x7D079EB1, 0xF00F9344,
    0x8708A3D2, 0x1E01F268, 0x6906C2FE, 0xF762575D, 0x806567CB,
    0x196C3671, 0x6E6B06E7, 0xFED41B76, 0x89D32BE0, 0x10DA7A5A,
    0x67DD4ACC, 0xF9B9DF6F, 0x8EBEEFF9, 0x17B7BE43, 0x60B08ED5,
    0xD6D6A3E8, 0xA1D1937E, 0x38D8C2C4, 0x4FDFF252, 0xD1BB67F1,
    0xA6BC5767, 0x3FB506DD, 0x48B2364B, 0xD80D2BDA, 0xAF0A1B4C,
    0x36034AF6, 0x41047A60, 0xDF60EFC3, 0xA867DF55, 0x316E8EEF,
    0x4669BE79, 0xCB61B38C, 0xBC66831A, 0x256FD2A0, 0x5268E236,
    0xCC0C7795, 0xBB0B4703, 0x220216B9, 0x5505262F, 0xC5BA3BBE,
    0xB2BD0B28, 0x2BB45A92, 0x5CB36A04, 0xC2D7FFA7, 0xB5D0CF31,
    0x2CD99E8B, 0x5BDEAE1D, 0x9B64C2B0, 0xEC63F226, 0x756AA39C,
    0x026D930A, 0x9C0906A9, 0xEB0E363F, 0x72076785, 0x05005713,
    0x95BF4A82, 0xE2B87A14, 0x7BB12BAE, 0x0CB61B38, 0x92D28E9B,
    0xE5D5BE0D, 0x7CDCEFB7, 0x0BDBDF21, 0x86D3D2D4, 0xF1D4E242,
    0x68DDB3F8, 0x1FDA836E, 0x81BE16CD, 0xF6B9265B, 0x6FB077E1,
    0x18B74777, 0x88085AE6, 0xFF0F6A70, 0x66063BCA, 0x11010B5C,
    0x8F659EFF, 0xF862AE69, 0x616BFFD3, 0x166CCF45, 0xA00AE278,
    0xD70DD2EE, 0x4E048354, 0x3903B3C2, 0xA7672661, 0xD06016F7,
    0x4969474D, 0x3E6E77DB, 0xAED16A4A, 0xD9D65ADC, 0x40DF0B66,
    0x37D83BF0, 0xA9BCAE53, 0xDEBB9EC5, 0x47B2CF7F, 0x30B5FFE9,
    0xBDBDF21C, 0xCABAC28A, 0x53B39330, 0x24B4A3A6, 0xBAD03605,
    0xCDD70693, 0x54DE5729, 0x23D967BF, 0xB3667A2E, 0xC4614AB8,
    0x5D681B02, 0x2A6F2B94, 0xB40BBE37, 0xC30C8EA1, 0x5A05DF1B,
    0x2D02EF8D
};

// We keep this around if we need to do consistency checks for arm/x64 builds
static U32 CRC(const void* src, int n_bytes)
{
    U32 C;

    C = 0xFFFFFFFF;

    {
        unsigned char const *S = (unsigned char const *) src;
        while (n_bytes--)
        {
            C = CRC32Table[((int) (C) ^ (*S++)) & 0xFF] ^ ((C) >> 8);
        }
    }

    C ^= 0xFFFFFFFF;
    return C;
}

//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
static void MemBufferAdd(MemBuffer* mem, void* data, U32 data_len)
{
    MemBufferEntry* entry = (MemBufferEntry*)mem->memalloc(sizeof(MemBufferEntry) + data_len);
    entry->bytecount = data_len;
    entry->next = 0;
    memcpy(entry->bytes, data, data_len);

    mem->total_bytes += data_len;
    if (mem->head == 0)
    {
        mem->tail = entry;
        mem->head = entry;
    }
    else
    {
        mem->tail->next = entry;
        mem->tail = entry;
    }
}

//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
static void MemBufferFree(MemBuffer* mem)
{
    MemBufferEntry* entry = mem->head;
    while (entry)
    {
        MemBufferEntry* next = entry->next;
        mem->memfree(entry);
        entry = next;
    }
    mem->total_bytes = 0;
    mem->head = 0;
    mem->tail = 0;
}


//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
static void MemBufferWriteBuffer(MemBuffer* mem, void* buffer)
{
    char* write_cursor = (char*)buffer;
    MemBufferEntry* entry = mem->head;
    while (entry)
    {
        MemBufferEntry* next = entry->next;
        memcpy(write_cursor, entry->bytes, entry->bytecount);
        write_cursor += entry->bytecount;
        entry = next;
    }
}

#define SEEK_TABLE_BUFFER_CACHE 16
//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
struct SeekTableBuffer
{
    MemBuffer buffer;
    U16 current_stack[SEEK_TABLE_BUFFER_CACHE];
    U32 current_index;
    U32 total_count;
    void* collapsed;
};

//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
static void SeekTableBufferAdd(SeekTableBuffer* seek, U16 entry)
{
    seek->current_stack[seek->current_index] = entry;
    seek->current_index++;
    if (seek->current_index >= SEEK_TABLE_BUFFER_CACHE)
    {
        MemBufferAdd(&seek->buffer, seek->current_stack, sizeof(seek->current_stack));
        seek->current_index = 0;
    }
    seek->total_count++;
}

//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
static void SeekTableBufferFree(SeekTableBuffer* seek)
{
    seek->buffer.memfree(seek->collapsed);
    MemBufferFree(&seek->buffer);
    seek->current_index = 0;
    seek->total_count = 0;
}

//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
static U32 SeekTableBufferTrim(SeekTableBuffer* seek, U16 max_entry_count)
{
    // Return # of frames per entry.

    // resolve to a single buffer for ease.
    U16* table = (U16*)seek->buffer.memalloc(seek->total_count * sizeof(U16));
    MemBufferWriteBuffer(&seek->buffer, table);
    memcpy((char*)table + seek->buffer.total_bytes, seek->current_stack, sizeof(U16) * seek->current_index);

    U32 frames_per_entry = 1;
    while (seek->total_count > (U32)max_entry_count)
    {
        // collapse pairs.
        frames_per_entry <<= 1;

        U32 read_index = 0;
        U32 write_index = 0;
        while (read_index < seek->total_count)
        {
            U32 total = table[read_index];
            if (read_index + 1 < seek->total_count)
            {
                total += table[read_index + 1];
            }

            if (total > 65535)
            {
                // we can't fit the offset in the table - fail to trim, this file is too big
                // with the given max seek table size.
                return ~0U;
            }

            table[write_index] = (U16)total;
            write_index++;
            read_index += 2;
        }

        seek->total_count = write_index;
    }

    seek->collapsed = table;
    return frames_per_entry;
}

// Scope guard to set up FP state as desired and reset it on exit
struct FPStateScope
{
    U32 saved_state;

    FPStateScope();
    ~FPStateScope();
};

#if defined(__RADSSE2__)
    FPStateScope::FPStateScope()
    {
        saved_state = _mm_getcsr();
        // Set up our expected FP state: no exception flags set,
        // all exceptions masked (suppressed), round to nearest,
        // flush to zero and denormals are zero both off.
        _mm_setcsr(_MM_MASK_MASK /* all exceptions masked */ | _MM_ROUND_NEAREST | _MM_FLUSH_ZERO_OFF);
    }

    FPStateScope::~FPStateScope()
    {
        _mm_setcsr(saved_state);
    }

#elif defined(__RADARM__) && defined(__RAD64__)

    #ifdef _MSC_VER

        #include <intrin.h>
        static U32 read_fpcr()
        {
            // The system register R/W instructions use 64-bit GPRs, but the
            // architectural FPCR is 32b
            return (U32)_ReadStatusReg(ARM64_FPCR);
        }

        static void write_fpcr(U32 state)
        {
            _WriteStatusReg(ARM64_FPCR, state);
        }

    #elif defined(__clang__) || defined(__GNUC__)

        static U32 read_fpcr()
        {
            // The system register R/W instructions use 64-bit GPRs, but the
            // architectural FPCR is 32b
            U64 value;
            __asm__ volatile("mrs %0, fpcr" : "=r"(value));
            return (U32)value;
        }

        static void write_fpcr(U32 state)
        {
            U64 state64 = state;
            __asm__ volatile("msr fpcr, %0" : : "r"(state64));
        }

    #else

        #error compiler? Not clang or msvc

    #endif

    FPStateScope::FPStateScope()
    {
        saved_state = read_fpcr();

        // IEEE compliant mode in FPCR is just all-0
        write_fpcr(0);
    }

    FPStateScope::~FPStateScope()
    {
        write_fpcr(saved_state);
    }

#else // neither SSE2 nor ARM64

    FPStateScope::FPStateScope()
        : saved_state(0)
    {
    }

    FPStateScope::~FPStateScope()
    {
    }

#endif

//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
uint8_t UECompressBinkAudio(
    void* WavData, uint32_t WavDataLen, uint32_t WavRate, uint8_t WavChannels, 
    uint8_t Quality, uint8_t GenerateSeekTable, uint16_t SeekTableMaxEntries, BAUECompressAllocFnType* MemAlloc, BAUECompressFreeFnType* MemFree,
    void** OutData, uint32_t* OutDataLen)
{
    FPStateScope FixFloatingPoint;

    if (WavChannels == 0)
        return BINKA_COMPRESS_ERROR_CHANS;
    if (WavRate > 256000 || WavRate < 2000)
        return BINKA_COMPRESS_ERROR_RATE;
    if (Quality > 9)
        return BINKA_COMPRESS_ERROR_QUALITY;
    if (MemAlloc == nullptr ||
        MemFree == nullptr)
        return BINKA_COMPRESS_ERROR_ALLOCATORS;
    if (OutData == nullptr ||
        OutDataLen == nullptr)
        return BINKA_COMPRESS_ERROR_OUTPUT;
    if (GenerateSeekTable && SeekTableMaxEntries < 2)
        return BINKA_COMPRESS_ERROR_SEEKTABLE;

    //
    // Deinterlace the input.
    //
    U32 SamplesPerChannel = WavDataLen / (sizeof(S16) * WavChannels);
    U8 NumBinkStreams = (WavChannels / 2) + (WavChannels  & 1);
    if (SamplesPerChannel == 0)
        return BINKA_COMPRESS_ERROR_SAMPLES;
    if (NumBinkStreams > MAX_STREAMS)
        return BINKA_COMPRESS_ERROR_CHANS;

    S32 ChannelsPerStream[MAX_STREAMS] = {};
    char* SourceStreams[MAX_STREAMS] = {};
    S32 BytesPerStream[MAX_STREAMS] = {};
    S32 CurrentDeintChannel = 0;
    for (S32 i = 0; i < NumBinkStreams; i++)
    {
        ChannelsPerStream[i] = (WavChannels - CurrentDeintChannel) > 1 ? 2 : 1;
        BytesPerStream[i] = ChannelsPerStream[i] * sizeof(S16) * SamplesPerChannel;
        SourceStreams[i] = (char*)MemAlloc(BytesPerStream[i]);

        S32 InputStride = sizeof(S16) * WavChannels;
        char* pInput = (char*)WavData + CurrentDeintChannel*sizeof(S16);
        char* pOutput = SourceStreams[i];
        char* pInputEnd = pInput + SamplesPerChannel*InputStride;
        

        if (ChannelsPerStream[i] == 2)
        {
            while (pInput < pInputEnd) // less than to protect against malformed wavs
            {
                *(S32*)pOutput = *(S32*)pInput;
                pInput += InputStride;
                pOutput += 2*sizeof(S16);
            }
        }
        else
        {
            while (pInput < pInputEnd) // less than to protect against malformed wavs
            {
                *(S16*)pOutput = *(S16*)pInput;
                pInput += InputStride;
                pOutput += sizeof(S16);
            }
        }
        CurrentDeintChannel += ChannelsPerStream[i];
    }

    //
    // We have a number of streams that need to be encoded.
    //
    char* StreamCursors[MAX_STREAMS];
    for (int i = 0; i < MAX_STREAMS; i++)
        StreamCursors[i] = SourceStreams[i];

    S32 StreamBytesGenerated[MAX_STREAMS] = {0};

    HBINKAUDIOCOMP hBink[MAX_STREAMS] = {0};
    for (S32 i = 0; i < NumBinkStreams; i++)
    {
        // the fn pointer cast is because on linux uintptr_t is unsigned long
        // when UINTa is unsigned long long. They have the same size, but are
        // technically different types.
        hBink[i] = BinkAudioCompressOpen(WavRate, ChannelsPerStream[i], BINKAC20, (BinkAudioCompressAllocFnType*)MemAlloc, MemFree);
    }

    SeekTableBuffer SeekTable = {};
    MemBuffer DataBuffer = {};
    DataBuffer.memalloc = MemAlloc;
    DataBuffer.memfree = MemFree;
    SeekTable.buffer.memalloc = MemAlloc;
    SeekTable.buffer.memfree = MemFree;

    U32 LastFrameLocation = 0;
    S32 MaxBlockSize = 0;

    for (;;)
    {
        void* InputBuffers[MAX_STREAMS];
        U32 InputLens[MAX_STREAMS];
        U32 OutputLens[MAX_STREAMS];
        void* OutputBuffers[MAX_STREAMS];
        U32 InputUseds[MAX_STREAMS];

        //
        // Run the compression for all of the streams at once.
        //
        U32 LimitedToSamples = ~0U;
        for (S32 StreamIndex = 0; StreamIndex < NumBinkStreams; StreamIndex++)
        {
            BinkAudioCompressLock(hBink[StreamIndex], InputBuffers + StreamIndex, InputLens + StreamIndex);

            // Copy only what we have remaining over. We could be zero filling at this point.
            U32 RemainingBytesInStream = (U32)(BytesPerStream[StreamIndex] - (StreamCursors[StreamIndex] - SourceStreams[StreamIndex]));

            U32 CopyAmount;
            {
                CopyAmount = InputLens[StreamIndex];
                if (RemainingBytesInStream < CopyAmount)
                    CopyAmount = RemainingBytesInStream;
            }

            memcpy(InputBuffers[StreamIndex], StreamCursors[StreamIndex], CopyAmount);

            // Zero the rest of the buffer, in case the last frame needs it.
            memset((char*)InputBuffers[StreamIndex] + CopyAmount, 0, InputLens[StreamIndex] - CopyAmount);

            // Do the actual compression.
            BinkAudioCompressUnlock(
                hBink[StreamIndex], 
                Quality, 
                InputLens[StreamIndex], 
                OutputBuffers + StreamIndex,
                OutputLens + StreamIndex,
                InputUseds + StreamIndex);

            //printf("Block In: %08x Out: %08x\n", CRC(InputBuffers[StreamIndex], InputLens[StreamIndex]), CRC(OutputBuffers[StreamIndex], OutputLens[StreamIndex]));
        }

        // Update cursors.
        S32 AllDone = 1;
        {
            for (S32 i = 0; i < NumBinkStreams; i++)
            {
                StreamCursors[i] += InputLens[i];
                if (StreamCursors[i] > SourceStreams[i] + BytesPerStream[i])
                    StreamCursors[i] = SourceStreams[i] + BytesPerStream[i];

                StreamBytesGenerated[i] += InputUseds[i];
                if (StreamBytesGenerated[i] < BytesPerStream[i]) 
                    AllDone = 0;
                else if (StreamBytesGenerated[i] > BytesPerStream[i])
                {
                    // We generated more samples that necessary - trim the
                    // block. We need to know how many of the samples this
                    // frame generated were valid - so subtract back off 
                    // to get where we ended last frame and take that from
                    // the len of the stream.
                    LimitedToSamples = (BytesPerStream[i] - (StreamBytesGenerated[i] - InputUseds[i])) >> 1;
                    if (ChannelsPerStream[i] == 2)
                        LimitedToSamples >>= 1;
                }
            }
        }

        // Figure how big this block is.
        S32 TotalBytesUsedForBlock = 0;
        for (S32 i = 0; i < NumBinkStreams; i++)
            TotalBytesUsedForBlock += OutputLens[i];

        // Write the header for the block.
        if (LimitedToSamples == ~0U)
        {
            U32 BlockHeader = (TotalBytesUsedForBlock << 16) | (0x9999);
            MemBufferAdd(&DataBuffer, &BlockHeader, 4);
        }
        else
        {
            U32 BlockHeader = 0xffff0000 | 0x9999;
            MemBufferAdd(&DataBuffer, &BlockHeader, 4);
            U32 LimitHeader = (LimitedToSamples << 16) | TotalBytesUsedForBlock;
            MemBufferAdd(&DataBuffer, &LimitHeader, 4);
        }

        // Write the compressed data.
        for (S32 i = 0; i < NumBinkStreams; i++)
            MemBufferAdd(&DataBuffer, OutputBuffers[i], OutputLens[i]);

        // Figure stats.
        if (TotalBytesUsedForBlock > MaxBlockSize) 
            MaxBlockSize = TotalBytesUsedForBlock;

        // Write the seek table to the actual output.
        if (GenerateSeekTable)
        {
            SeekTableBufferAdd(&SeekTable, (U16)(DataBuffer.total_bytes - LastFrameLocation));
            LastFrameLocation = DataBuffer.total_bytes;
        }

        if (AllDone) 
            break;
    }

    uint8_t Result = BINKA_COMPRESS_SUCCESS;
    // Trim the table and get how many frames ended up per entry.
    U32 FramesPerEntry = SeekTableBufferTrim(&SeekTable, SeekTableMaxEntries);
    if (FramesPerEntry == ~0U)
    {
        // The file is too big - can't fit offsets in to U16.
        Result = BINKA_COMPRESS_ERROR_SIZE;
    }
    else
    {
        BinkAudioFileHeader Header;
        Header.tag = 'UEBA';
        Header.PADDING = 0;
        Header.channels = (U8)WavChannels;
        Header.rate = WavRate;
        Header.sample_count = SamplesPerChannel;
        Header.max_comp_space_needed = (U16)MaxBlockSize;
        Header.flags = 1;
        Header.version = 1;
        Header.output_file_size = DataBuffer.total_bytes + sizeof(BinkAudioFileHeader) + SeekTable.total_count*sizeof(U16);
        Header.blocks_per_seek_table_entry = (U16)FramesPerEntry;
        Header.seek_table_entry_count = (U16)SeekTable.total_count;

        // Create the actual output buffer.
        char* Output = (char*)MemAlloc(Header.output_file_size);
        memcpy(Output, &Header, sizeof(BinkAudioFileHeader));
        memcpy(Output + sizeof(BinkAudioFileHeader), SeekTable.collapsed, SeekTable.total_count * sizeof(U16));
        MemBufferWriteBuffer(&DataBuffer, Output + sizeof(BinkAudioFileHeader) + SeekTable.total_count * sizeof(U16));

        *OutData = Output;
        *OutDataLen = Header.output_file_size;

        Result = BINKA_COMPRESS_SUCCESS;
    }

    MemBufferFree(&DataBuffer);
    SeekTableBufferFree(&SeekTable);
    for (S32 i = 0; i < NumBinkStreams; i++)
    {
        MemFree(SourceStreams[i]);
        BinkAudioCompressClose(hBink[i]);
    }


    return Result;
}

#if 0
#include <malloc.h>

struct fuzzer_input
{
    uint32_t Rate;
    uint8_t Chans;
    uint8_t Quality;
    uint8_t GenTable;
    uint16_t GenTableMaxEntries;  
};

static void* AllocThunk(uintptr_t ByteCount) { return malloc(ByteCount); }
static void FreeThunk(void* Ptr) { free(Ptr); }

extern "C" int LLVMFuzzerTestOneInput(const uint8_t *Data, size_t Size) 
{    
    if (Size <= sizeof(fuzzer_input))
        return 0;
    if (Size > 0xFFFFFFFFU)
        return 0;

    fuzzer_input* input = (fuzzer_input*)Data;

    void* CompressedData = 0;
    uint32_t CompressedLen;

    UECompressBinkAudio((void*)(Data + sizeof(fuzzer_input)), (uint32_t)Size - sizeof(fuzzer_input),
        input->Rate, input->Chans, input->Quality, input->GenTable, input->GenTableMaxEntries, AllocThunk, FreeThunk,
        &CompressedData, &CompressedLen);

    FreeThunk(CompressedData);

    return 0;
}
#endif
