// Copyright Epic Games, Inc. All Rights Reserved.

#include "binka_ue_encode.h"

#include <stdio.h>
#include <stdlib.h>

static void* allocator_thunk(uintptr_t bytes)
{
    return malloc(bytes);
}
static void free_thunk(void* ptr)
{
    free(ptr);
}

int main(int argc, char** argv)
{
    int16_t data[48000] = {};

    void* Compressed;
    uint32_t CompressedLen;
    UECompressBinkAudio(data, 48000*2, 48000, 1, 0, 1, 4096, allocator_thunk, free_thunk, &Compressed, &CompressedLen);

    printf("Compressed %d %d\n", 48000*2, CompressedLen);

    return 0;
}
