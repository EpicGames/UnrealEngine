// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef __RADRR_COREH__
  #include "rrCore.h"
#endif

#include <string.h>
#include <stdint.h>

#include "radmath.h"
#include "binkacd.h"
#include "cpu.h"

#include "radfft.h"

#ifndef alloca
#if defined(_MSC_VER)
  #include <malloc.h>
#elif defined(__GNUC__)
  #include <alloca.h>
#endif
#endif


#ifdef BIG_OLE_FFT // always false for ue binka
#define MAX_TRANSFORM                 4096
#else
#define MAX_TRANSFORM                 2048
#endif

#define MAXCHANNELS          2
#define WINDOWRATIO          16

#define TOTBANDS 25

#define FXPBITS 29

#define VQLENGTH 8

#define RLEBITS 4
#define MAXRLE ( 1 << RLEBITS )

static
U8 bink_rlelens_snd[ MAXRLE ] =
{
  2,3,4,5, 6,8,9,10, 11,12,13,14, 15,16,32,64
};

static
U32 bink_bandtopfreq[ TOTBANDS ]=
{
  0, 100, 200, 300, 400, 510, 630, 770, 920, 1080, 1270, 1480, 1720, 2000,
  2320, 2700, 3150, 3700, 4400, 5300, 6400, 7700, 9500, 12000, 15500
};


#include "undeci.inc"

/*static F32 RADINLINE Undecibel( F32 d )
{
  return( ( F32 ) radpow( 10, d * 0.10f ) );
}
*/

//==============================================================================
//   decoding functions
//==============================================================================

static
F32 bink_invertbins[ 24 ]=
{
  (F32)( 1.0F / (F32)( 1 << 23 ) ),
  (F32)( 1.0F / (F32)( 1 << 22 ) ),(F32)( 1.0F / (F32)( 1 << 21 ) ),(F32)( 1.0F / (F32)( 1 << 20 ) ),(F32)( 1.0F / (F32)( 1 << 19) ),
  (F32)( 1.0F / (F32)( 1 << 18 ) ),(F32)( 1.0F / (F32)( 1 << 17 ) ),(F32)( 1.0F / (F32)( 1 << 16 ) ),(F32)( 1.0F / (F32)( 1 << 15) ),
  (F32)( 1.0F / (F32)( 1 << 14 ) ),(F32)( 1.0F / (F32)( 1 << 13 ) ),(F32)( 1.0F / (F32)( 1 << 12 ) ),(F32)( 1.0F / (F32)( 1 << 11) ),
  (F32)( 1.0F / (F32)( 1 << 10 ) ),(F32)( 1.0F / (F32)( 1 <<  9 ) ),(F32)( 1.0F / (F32)( 1 <<  8 ) ),(F32)( 1.0F / (F32)( 1 <<  7) ),
  (F32)( 1.0F / (F32)( 1 <<  6 ) ),(F32)( 1.0F / (F32)( 1 <<  5 ) ),(F32)( 1.0F / (F32)( 1 <<  4 ) ),(F32)( 1.0F / (F32)( 1 <<  3) ),
  (F32)( 1.0F / (F32)( 1 <<  2 ) ),(F32)( 1.0F / (F32)( 1 <<  1 ) ),(F32)( 1.0F / (F32)( 1 <<  0 ) )
};

#if defined(__RADPPC__) && defined(_MSC_VER)
#pragma optimize ("g", off)
#endif

static F32 fxptof( U32 val )
{
  F32 f;

  f = (F32) ( ( (F32) (S32) ( ( val & ( ~0x10000000 ) ) >> 5 ) ) * bink_invertbins[ val & 31 ] );
  return( ( val & 0x10000000 ) ? -f : f );
}

#if defined(__RADPPC__) && defined(_MSC_VER)
#pragma optimize ("g", on)
#endif

typedef struct BINKAUDIODECOMP
{
  S16 * overlap;

  U32 transform_size;
  F32 transform_size_root;

  U32 buffer_size;
  U32 window_size_in_bytes;

  U32 window_shift;
  U32 chans;

  S32 start_frame;
  U32 num_bands;

  U32 _unused; // work_sz used to be here
  U32 flags;

  U32 size;
  U32 bands[ TOTBANDS + 1 ];
} BINKAUDIODECOMP;

#include "crsfade.inl"

#include "binkbits.h"

#if  defined(__RADARM64__) && !defined(_M_ARM64) // exclude visual studio arm64 as it doesn't support __int128
  #define bigreg __int128
  #define bigregzero 0
  #define bigregloadaligned( val, ptr ) (val)=*((bigreg*)(ptr))
  #define bigregstorealigned( ptr, val ) *((bigreg*)(ptr))=(val)
  #define bigregstoreunaligned( ptr, val ) *((bigreg*)(ptr))=(val)
#elif defined(__RADNEON__)
  #include <arm_neon.h>
  #define bigreg uint8x16_t 
  #define bigregzero vdupq_n_s32(0)
  typedef RAD_ALIGN(uint8_t, ALU8,16);
  typedef RAD_ALIGN(uint8_t, ULU8,1);
  #define bigregloadaligned( val, ptr ) (val)=vld1q_u8( ((ALU8*)(ptr)) )
  #define bigregstorealigned( ptr, val ) vst1q_u8( ((ALU8*)(ptr)),val)
  #define bigregstoreunaligned( ptr, val ) vst1q_u8( ((ULU8*)(ptr)),val)
#elif defined(__RADARM__)
  typedef struct bigreg { U32 a,b; } bigreg;  // alignment trick
  #define bigreg bigreg
  #define bigregzero {0}
  #define bigregloadaligned( val, ptr ) (val)=*((bigreg*)(ptr))
  #define bigregstorealigned( ptr, val ) *((bigreg*)(ptr))=(val)
  #define bigregstoreunaligned( ptr, val ) *((bigreg*)(ptr))=(val)
#elif defined(__RADX86__)
  #include <xmmintrin.h>
  #define bigreg __m128
  #define bigregzero _mm_setzero_ps()
  #define bigregloadaligned( val, ptr ) (val)=_mm_load_ps( ((float*)(ptr)) )
  #define bigregstorealigned( ptr, val ) _mm_store_ps( ((float*)(ptr)),val)
  #define bigregstoreunaligned( ptr, val ) _mm_storeu_ps( ((float*)(ptr)),val)
#else
  #define rrmemsetzero(d,c) memset(d,0,c) // use for small zero clears
  #define ourmemsetzero rrmemsetzero
  #define rrmemmovebig memmove // use for large copies (>512 bytes) - can overlay
  #define ourmemcpy rrmemmovebig
#endif

#ifdef bigreg

static void ourmemcpy( void * destp, void const * srcp, U32 bytes ) //assumed aligned and multiple of 16
{
  #define PERLOOP (4*sizeof(bigreg))
  U32 s = bytes/PERLOOP;
  U8 const * src = (U8 const*)srcp;
  U8 * dest = (U8*)destp;
  rrassert( (bytes&15)==0 );
  bytes = ( bytes & 63 ) / 16;
  rrassert( (((UINTa)srcp)&15)==0);
  rrassert( (((UINTa)destp)&15)==0);
  rrassert( srcp != destp );
  while(s) // 64-byte chunks
  {
    bigreg a,b,c,d;
#ifdef __clang__
  __asm__ __volatile__("");  // force no conversion to enormous optimized memset
#endif  
    bigregloadaligned(a,((bigreg*)src)); bigregloadaligned(b,((bigreg*)src)+1); bigregloadaligned(c,((bigreg*)src)+2); bigregloadaligned(d,((bigreg*)src)+3);
    bigregstorealigned(((bigreg*)dest),a); bigregstorealigned(((bigreg*)dest)+1,b); bigregstorealigned(((bigreg*)dest)+2,c); bigregstorealigned(((bigreg*)dest)+3,d);
    --s;
    src+=PERLOOP;
    dest+=PERLOOP;
  };
  #undef PERLOOP
  
  while ( bytes ) // 16-byte chunks
  {
    bigreg a;
    bigregloadaligned(a,((bigreg*)src)); bigregstorealigned(((bigreg*)dest),a);
    src+=16; 
    dest+=16;
    --bytes;
  }
}

// always writes 32 bytes (so possible 31 byte overwrite), except arm32 without neon, 16 bytes (possible 15 bytes overwrite)
//   addr can be unaligned
static void ourmemsetzero( void * addr, size_t len )
{
  bigreg z = bigregzero;

  #define PERLOOP (2*sizeof(bigreg))

  len = ( len + (PERLOOP-1) ) / PERLOOP;
  do
  {
    --len;
#ifdef __clang__
  __asm__ __volatile__("");  // force no conversion to enormous optimized memset
#endif  
    bigregstoreunaligned( ((float*)addr),z);
    bigregstoreunaligned( ((float*)addr)+4,z);
    addr = ((float*)addr)+8;
  } while(len);
  #undef PERLOOP
}

#endif


static RADINLINE U32 read_up_to_24_bits(const U8* buffer, size_t bit_position)
{
    uint32_t x;
    memcpy(&x, buffer + (bit_position >> 3), sizeof(x));
    return x >> (bit_position & 7);
}

template <int bitlen, int index>
static RADINLINE void decode_coeff(unsigned char& current_sign_bit, U64 coeff_bits, U32 sign_bits, short* coeffs)
{
    unsigned int need_negate;
    short c;

    // this is crafted to coerce clang to generate ubxt, csinc, and csneg.
    c = (coeff_bits >> (index * bitlen)) & ((1 << bitlen) - 1);
    need_negate = (sign_bits >> current_sign_bit) & 1;
    current_sign_bit += (c != 0) ? 1 : 0;
    c = need_negate ? -c : c;
    coeffs[index] = c;
}

template <int bitlen>
static RADINLINE size_t decode_coeff_remnants(size_t start, float* results, const uint8_t* buffer, size_t coeff_position, size_t sign_position, size_t count)
{
    // This should almost never be hit
    for (; start < count; start++)
    {
        S16 c;
        unsigned char current_sign_bit = 0;

        U32 coeff_bits = read_up_to_24_bits(buffer, coeff_position);
        coeff_position += bitlen;
        U32 sign_bits = read_up_to_24_bits(buffer, sign_position);

        decode_coeff<bitlen, 0>(current_sign_bit, coeff_bits, sign_bits, &c);
        sign_position += current_sign_bit;

        results[start] = (float)c;
    }

    return sign_position;
}

template <class chunk_type>
static RADINLINE bool validate_chunked_bit_read(size_t bit_position, size_t end_bit_position, size_t need_bits)
{
    //
    // The check here is because even if we only need 1 bit and that fits
    // within the end_bit_position, we'll be reading a lot more to fill the
    // chunk.
    //
    // this all simpifies a great deal with compiling.
    size_t farthest_byte_to_read = (sizeof(chunk_type)/8) + ((bit_position + need_bits) >> 3);
    size_t farthest_bit_to_read = farthest_byte_to_read * 8;
    return farthest_bit_to_read <= end_bit_position;
}


//#define FORCE_SCALAR

#ifdef __RAD64__

static RADINLINE U64 read_up_to_56_bits(const U8* buffer, size_t bit_position)
{
    uint64_t x;
    memcpy(&x, buffer + (bit_position >> 3), sizeof(x));
    return x >> (bit_position & 7);
}

#if defined(RAD_USES_SSSE3) && defined(__RADX64__)

#include <smmintrin.h>
#include <xmmintrin.h>
#include <emmintrin.h>

typedef __m128i Vec128;

static Vec128 load128u(const void * ptr) { return _mm_loadu_si128((const __m128i *) ptr); }

//static Vec128 s_getbit_consts[15][2]; // [bitlen-1][idx]   0=shuffle mask, 1=multipliers
//static volatile U32 s_getbit_consts_initd;

static RAD_ALIGN(U32, s_getbit_consts_table[], 16) =
{
  0x1000100, 0x1000100, 0x1000100, 0x1000100, 0x40408080, 0x10102020, 0x4040808, 0x1010202,
  0x1000100, 0x1000100, 0x2010201, 0x2010201, 0x20208080, 0x2020808, 0x20208080, 0x2020808,
  0x1000100, 0x2010100, 0x2010201, 0x3020302, 0x10108080, 0x40400202, 0x1010808, 0x4042020,
  0x1000100, 0x2010201, 0x3020302, 0x4030403, 0x8088080, 0x8088080, 0x8088080, 0x8088080,
  0x1000100, 0x2010201, 0x4030302, 0x5040403, 0x4048080, 0x1012020, 0x40400808, 0x10100202,
  0x1000100, 0x3020201, 0x4030403, 0x6050504, 0x2028080, 0x20200808, 0x2028080, 0x20200808,
  0x1000100, 0x3020201, 0x5040403, 0x7060605, 0x1018080, 0x4040202, 0x10100808, 0x40402020,
  0x2010100, 0x4030302, 0x6050504, 0x8070706, 0x80808080, 0x80808080, 0x80808080, 0x80808080,
  0x2010100, 0x4030302, 0x6050504, 0x8070706, 0x40408080, 0x10102020, 0x4040808, 0x1010202,
  0x2010100, 0x4030302, 0x7060605, 0x9080807, 0x20208080, 0x2020808, 0x20208080, 0x2020808,
  0x2010100, 0x5040302, 0x7060605, 0xa090908, 0x10108080, 0x40400202, 0x1010808, 0x4042020,
  0x2010100, 0x5040403, 0x8070706, 0xb0a0a09, 0x8088080, 0x8088080, 0x8088080, 0x8088080,
  0x2010100, 0x5040403, 0x9080706, 0xc0b0a09, 0x4048080, 0x1012020, 0x40400808, 0x10100202,
  0x2010100, 0x6050403, 0x9080807, 0xd0c0b0a, 0x2028080, 0x20200808, 0x2028080, 0x20200808,
  0x2010100, 0x6050403, 0xa090807, 0xe0d0c0b, 0x1018080, 0x4040202, 0x10100808, 0x40402020,
};

//
//static void init_bit_lookup_tables()
//{
//    if (rrAtomicAddExchange32(&s_getbit_consts_initd, 1) == 0)
//    {
//        int bitlen, lane;
//        // Init bit decode table
//        for (bitlen = 1; bitlen <= 15; bitlen++)
//        {
//            uint8_t shuffle[16];
//            uint16_t mul[8];
//
//            for (lane = 0; lane < 8; ++lane)
//            {
//                int lane_bitpos = lane*bitlen;
//                shuffle[lane*2 + 0] = (U8)(lane_bitpos >> 3);
//                shuffle[lane*2 + 1] = (U8)((lane_bitpos >> 3) + 1);
//                mul[lane] = 0x8080 >> (lane_bitpos & 7);
//            }
//
//            s_getbit_consts[bitlen-1][0] = load128u(shuffle);
//            s_getbit_consts[bitlen-1][1] = load128u(mul);
//
//            printf("  0x%x, 0x%x, 0x%x, 0x%x, 0x%x, 0x%x, 0x%x, 0x%x,\n",
//                (shuffle[0] << 0) | (shuffle[1] << 8) | (shuffle[2] << 16) | (shuffle[3] << 24),
//                (shuffle[4] << 0) | (shuffle[5] << 8) | (shuffle[6] << 16) | (shuffle[7] << 24),
//                (shuffle[8] << 0) | (shuffle[9] << 8) | (shuffle[10] << 16) | (shuffle[11] << 24),
//                (shuffle[12] << 0) | (shuffle[13] << 8) | (shuffle[14] << 16) | (shuffle[15] << 24),
//                (mul[0] << 0) | (mul[1] << 16),
//                (mul[2] << 0) | (mul[3] << 16),
//                (mul[4] << 0) | (mul[5] << 16),
//                (mul[6] << 0) | (mul[7] << 16));
//        }
//    }
//}





static RADINLINE void determine_shuffle_and_mul(Vec128 * out_shuffle, Vec128 * out_mul, int bitlen, size_t bitpos)
{
    Vec128 extra_shift = _mm_cvtsi32_si128(bitpos & 7);
    Vec128 shuffle = _mm_load_si128( (__m128i*)((unsigned char*)s_getbit_consts_table + sizeof(__m128i)*2*(bitlen - 1)));
    Vec128 mul = _mm_load_si128((__m128i*)((unsigned char*)s_getbit_consts_table + sizeof(__m128i) * 2 * (bitlen - 1) + 1 * sizeof(__m128i)));
    Vec128 mul_lobyte, mul_byte_advanced;
    
    // Shift down the multiplier to consume the initial bits
    mul = _mm_srl_epi16(mul, extra_shift);

    // If shifting mul consumed the first byte, adjust shuffle accordingly.
    // Our original multiplier is 0x8080 >> (bitpos & 7); thus, when the
    // high 8 bits become 0, we know we consumed more than 8 bits total and
    // need to update the shuffle bytes.
    //
    // We test for "high byte 0" as x == (x & 0xff) since we need (x & 0xff)
    // anyway.
    mul_lobyte = _mm_and_si128(mul, _mm_set1_epi16(0xff));
    mul_byte_advanced = _mm_cmpeq_epi16(mul, mul_lobyte);

    // The 16-bit lanes where we went outside the original byte now have -1
    // in them; bytewise add to shuffle mask to increment the shuffle bytes
    // when this is the case.
    shuffle = _mm_sub_epi8(shuffle, mul_byte_advanced);

    *out_shuffle = shuffle;

    // Our output mult needs an extra shift-left by 1 which is just adding it to itself
    *out_mul = _mm_add_epi16(mul_lobyte, mul_lobyte);
}

// Inclusive prefix sum on U16 lanes
RAD_USES_SSSE3 static RADINLINE Vec128 prefix_sum_u16(Vec128 x)
{
    // Two Kogge-Stone steps, then finish with Sklansky
    // this has 2 shifts 1 shuffle vs. 3 shuffles for pure Kogge-Stone
    x = _mm_add_epi16(x, _mm_slli_epi64(x, 16));
    x = _mm_add_epi16(x, _mm_slli_epi64(x, 32));
    x = _mm_add_epi16(x, _mm_shuffle_epi8(x, _mm_setr_epi8(-1,-1,-1,-1,-1,-1,-1,-1, 6,7,6,7,6,7,6,7)));
    return x;
}

RAD_USES_SSSE3 static RADINLINE void decode_pfxsum_8(float* results, const uint8_t* buffer, Vec128 shuffle0, Vec128 shuffle1, Vec128 mul_vec, Vec128 mask_lut, Vec128 coeff_mask, Vec128* last_prefix_sum, size_t bitpos_vals, size_t bitpos_signs, int bitlen)
{
    const uint8_t* val_bytes = buffer + (bitpos_vals >> 3);
    Vec128 packed_coeffs = load128u(val_bytes);
    Vec128 coeff_bits;

    // Align target bits so bit 0 of our desired field is in bit 8 of the 16-bit lane
    Vec128 aligned = _mm_mullo_epi16(_mm_shuffle_epi8(packed_coeffs, shuffle0), mul_vec);
    coeff_bits = _mm_srli_epi16(aligned, 8);

    if (bitlen > 8)
    {
        // Need to grab the second byte and merge
        // this one already ends up in the high byte
        // no extra masking required, except for "truncation" due to shifting bits out
        // the two values agree
        aligned = _mm_mullo_epi16(_mm_shuffle_epi8(packed_coeffs, shuffle1), mul_vec);
        coeff_bits = _mm_or_si128(coeff_bits, aligned);
    }

    // Mask the active bits to finalize bit get
    {
        Vec128 unpacked_coeffs = _mm_and_si128(coeff_bits, coeff_mask);

        // Figure out which lanes are zero so we can work out who needs sign bits
        Vec128 coeff_zero = _mm_cmpeq_epi16(unpacked_coeffs, _mm_setzero_si128());
        Vec128 need_signs = _mm_add_epi16(coeff_zero, _mm_set1_epi16(1)); // 0 if no sign required, 1 if sign consumed

        // Prefix sum to figure out where everything comes from
        Vec128 pfx_sum = prefix_sum_u16(need_signs);
        

        // Peek at the next (up to) 8 sign bits, replicate into vector
        int sign_peek_bits = read_up_to_24_bits(buffer, bitpos_signs) & 0xff;
        Vec128 vec_signs = _mm_shuffle_epi8(_mm_cvtsi32_si128(sign_peek_bits), _mm_setzero_si128());

        // Materialize bit masks to test against via LUT
        Vec128 sign_masks = _mm_shuffle_epi8(mask_lut, pfx_sum);

        // Test the sign bits
        Vec128 negate_mask = _mm_cmpeq_epi16(_mm_and_si128(vec_signs, sign_masks), sign_masks);

        // Apply the signs to 16-bit values
        Vec128 signed_coeffs = _mm_sub_epi16(_mm_xor_si128(unpacked_coeffs, negate_mask), negate_mask);

        // Sign-extend packed coeffs to 32 bits
        Vec128 coeffs32_0 = _mm_srai_epi32(_mm_unpacklo_epi16(signed_coeffs, signed_coeffs), 16);
        Vec128 coeffs32_1 = _mm_srai_epi32(_mm_unpackhi_epi16(signed_coeffs, signed_coeffs), 16);

        // Convert to float and store
        __m128 coeffsf_0 = _mm_cvtepi32_ps(coeffs32_0);
        __m128 coeffsf_1 = _mm_cvtepi32_ps(coeffs32_1);

        _mm_storeu_ps(results + 0, coeffsf_0);
        _mm_storeu_ps(results + 4, coeffsf_1);
        *last_prefix_sum = pfx_sum;
    }
}

static RADINLINE size_t decode_pfxsum(float* results, const uint8_t* buffer, size_t bitpos_vals, size_t bitpos_signs, size_t end_bit_position, size_t count, int bitlen)
{
    Vec128 last_prefix_sum;
    size_t i;
    size_t run_count = count & ~7;

    const Vec128 coeff_mask = _mm_set1_epi16((1 << bitlen) - 1);

    // This mask has lane i = 1 << (i-1), with 0 in lane 0. (Lanes 9-15 ignored.)
    // -128 instead of 128 because the arguments to setr_epi8 are _signed_ int8.
    const Vec128 mask_lut = _mm_setr_epi8(0, 1, 2, 4, 8, 16, 32, 64, -128, 0, 0, 0, 0, 0, 0, 0);

    Vec128 shuffle0, shuffle1, mul_vec;
    

    determine_shuffle_and_mul(&shuffle0, &mul_vec, bitlen, bitpos_vals);

    shuffle1 = _mm_add_epi8(shuffle0, _mm_set1_epi8(1));
        
    for (i = 0; i < run_count; i +=8)
    {
        size_t nsigns;
        decode_pfxsum_8(results + i, buffer, shuffle0, shuffle1, mul_vec, mask_lut, coeff_mask, &last_prefix_sum, bitpos_vals, bitpos_signs, bitlen);

        // Advance read cursors
        bitpos_vals += 8 * bitlen;

        //
        // This extract doesn't actually need the &15, as the prefix sum can only ever be
        // a max of 8, however we have crash reports in the wild of situations where bitpos_signs
        // has gone stupid high and causing an AV. This happens after the initial run, and is _well_
        // off in to another page, so the belief is that we're getting some CPU overclocking
        // absurdity. This is confirmed because in the crash minidump we can see that the prefix
        // sum is sane. Since we don't know where the issue is, we're adding the &15. This ends up
        // replacing the mov ecx, ecx for sign extension in a 1:1 trade, so it's completely free.
        //
        // Then, to help sanitize, we test the bit positions against the end of the buffer and
        // just bail since we're off the edge of the map anyway. This ends up getting caught by
        // the bit position validation in read_channel_data_2 and zeroing the block.
        //
        nsigns = (size_t)(unsigned int)(_mm_extract_epi16(last_prefix_sum, 7) & 0xf);
        bitpos_signs += nsigns;

        if (bitpos_signs > end_bit_position ||
            bitpos_vals > end_bit_position)
        {
            return end_bit_position;
        }
    }

    // this should almost never happen - only when a run passes the end of the transform size.
    // as far as i can tell this only happens with zeroing runs
    if (count & 7)
    {
        float stack_results[8];
        unsigned short pfxsum[8];
        
        count &= 7;
        
        decode_pfxsum_8(stack_results, buffer, shuffle0, shuffle1, mul_vec, mask_lut, coeff_mask, &last_prefix_sum, bitpos_vals, bitpos_signs, bitlen);
        
        _mm_storeu_si128((__m128i*)pfxsum, last_prefix_sum);

        bitpos_signs = bitpos_signs + pfxsum[count-1];
        bitpos_vals += bitlen * (U32)count;

        for (i = 0; i < count; i++)
            results[run_count + i] = stack_results[i];
    }

    return bitpos_signs;
}

#endif // __RADX64__ && SSSE3

#ifdef __RADARM__
static RADINLINE void short_to_float_4(float* output, short* input)
{
    int16x4_t s16_low = vld1_s16(input);
    int32x4_t s32_low = vmovl_s16(s16_low);
    float32x4_t f32_low = vcvtq_f32_s32(s32_low);
    vst1q_f32(output, f32_low);
}
#else
static RADINLINE void short_to_float_4(float* output, short* input)
{
    size_t i = 0;
    for (i = 0; i < 4; i++)
        output[i] = (float)input[i];
}
#endif // __RADARM__


template <int bitlen>
static RADINLINE size_t decode_coeffs_runlength_8(float* results, const uint8_t* buffer, size_t coeff_position, size_t sign_position, size_t count)
{
    // 8 ceoffs fits in the 56 bits we can get - so we grab chunks of 8
    size_t run_count = count & ~7;
    size_t run = 0;

    for (run = 0; run < run_count; run+=8)
    {
        S16 coeffs[8];
        unsigned char current_sign_bit = 0;
        U64 coeff_bits = read_up_to_56_bits(buffer, coeff_position);
        coeff_position += 8 * bitlen;

        U32 sign_bits = read_up_to_24_bits(buffer, sign_position);

        decode_coeff<bitlen, 0>(current_sign_bit, coeff_bits, sign_bits, coeffs);
        decode_coeff<bitlen, 1>(current_sign_bit, coeff_bits, sign_bits, coeffs);
        decode_coeff<bitlen, 2>(current_sign_bit, coeff_bits, sign_bits, coeffs);
        decode_coeff<bitlen, 3>(current_sign_bit, coeff_bits, sign_bits, coeffs);
        decode_coeff<bitlen, 4>(current_sign_bit, coeff_bits, sign_bits, coeffs);
        decode_coeff<bitlen, 5>(current_sign_bit, coeff_bits, sign_bits, coeffs);
        decode_coeff<bitlen, 6>(current_sign_bit, coeff_bits, sign_bits, coeffs);
        decode_coeff<bitlen, 7>(current_sign_bit, coeff_bits, sign_bits, coeffs);

        sign_position += current_sign_bit;                

        short_to_float_4(results + run, coeffs);
        short_to_float_4(results + run + 4, coeffs + 4);
    }

    return decode_coeff_remnants<bitlen>(run, results, buffer, coeff_position, sign_position, count);
}


template <int bitlen>
static RADINLINE size_t decode_coeffs_runlength_4(float* results, const uint8_t* buffer, size_t coeff_position, size_t sign_position, size_t count)
{
    // 4 ceoffs fits in the 56 bits we can get - so we grab chunks of 4
    size_t run_count = count & ~3;
    size_t run = 0;

    for (run = 0; run < run_count; run+=4)
    {
        S16 coeffs[4];
        unsigned char current_sign_bit = 0;
        U64 coeff_bits = read_up_to_56_bits(buffer, coeff_position);
        coeff_position += 4 * bitlen;

        U32 sign_bits = read_up_to_24_bits(buffer, sign_position);

        decode_coeff<bitlen, 0>(current_sign_bit, coeff_bits, sign_bits, coeffs);
        decode_coeff<bitlen, 1>(current_sign_bit, coeff_bits, sign_bits, coeffs);
        decode_coeff<bitlen, 2>(current_sign_bit, coeff_bits, sign_bits, coeffs);
        decode_coeff<bitlen, 3>(current_sign_bit, coeff_bits, sign_bits, coeffs);

        sign_position += current_sign_bit;                

        short_to_float_4(results + run, coeffs);
    }

    return decode_coeff_remnants<bitlen>(run, results, buffer, coeff_position, sign_position, count);
}


static U32 RADINLINE decode_scalar(float* results, const uint8_t* buffer, size_t coeff_position, size_t sign_position, size_t count, int bitlen)
{
    switch (bitlen)
    {
    case 1: /* 1*8 = 8 <= 56 */  return (U32)decode_coeffs_runlength_8<1>(results, buffer, coeff_position, sign_position, count);
    case 2: /* 2*8 = 16 <= 56 */ return (U32)decode_coeffs_runlength_8<2>(results, buffer, coeff_position, sign_position, count);
    case 3: /* 3*8 = 24 <= 56 */ return (U32)decode_coeffs_runlength_8<3>(results, buffer, coeff_position, sign_position, count);
    case 4: /* 4*8 = 32 <= 56 */ return (U32)decode_coeffs_runlength_8<4>(results, buffer, coeff_position, sign_position, count);
    case 5: /* 5*8 = 40 <= 56 */ return (U32)decode_coeffs_runlength_8<5>(results, buffer, coeff_position, sign_position, count);
    case 6: /* 6*8 = 48 <= 56 */ return (U32)decode_coeffs_runlength_8<6>(results, buffer, coeff_position, sign_position, count);
    case 7: /* 7*8 = 48 <= 56 */ return (U32)decode_coeffs_runlength_8<7>(results, buffer, coeff_position, sign_position, count);
    case 8: /* 8*4 = 32 <= 56 */ return (U32)decode_coeffs_runlength_4<8>(results, buffer, coeff_position, sign_position, count);
    case 9:  /* 9*4 = 36 <= 56 */ return (U32)decode_coeffs_runlength_4<9>(results, buffer, coeff_position, sign_position, count);
    case 10: /* 10*4 = 40 <= 56 */ return (U32)decode_coeffs_runlength_4<10>(results, buffer, coeff_position, sign_position, count);
    case 11: /* 11*4 = 44 <= 56 */ return (U32)decode_coeffs_runlength_4<11>(results, buffer, coeff_position, sign_position, count);
    case 12: /* 12*4 = 48 <= 56 */ return (U32)decode_coeffs_runlength_4<12>(results, buffer, coeff_position, sign_position, count);
    case 13: /* 13*4 = 52 <= 56 */ return (U32)decode_coeffs_runlength_4<13>(results, buffer, coeff_position, sign_position, count);
    case 14: /* 14*4 = 56 <= 56 */ return (U32)decode_coeffs_runlength_4<14>(results, buffer, coeff_position, sign_position, count);
    case 15:
        {
            // super rare and can't really group in a convenient way - just do as singles.
            return (U32)decode_coeff_remnants<15>(0, results, buffer, coeff_position, sign_position, count);
        }
    }
    // invalid data.
    return (U32)sign_position;
}



static void read_channel_data_2( F32 * samps,
                               U32 transform_size,
                               U32 num_bands,
                               BINKVARBITS * vbp, 
                               U32 * bands,
                               void * padded_in_data_end ) 
{
  size_t i = 0;
  size_t b;
  F32 threshold[TOTBANDS + 2];

  // convert from varbits to base + offset
   U8* buffer = ((U8*)vbp->cur) - 8;
  size_t bit_position = 64 - vbp->bitlen;
  U32 end_bit_position = (U32)(((U8*)padded_in_data_end - buffer) * 8);

  // validate we can get to the transform
  U32 bits_needed = num_bands * 8 + FXPBITS * 2;
  if (validate_chunked_bit_read<uint32_t>(bit_position, end_bit_position, bits_needed) == false)
  {
    // corrupted data.
read_chan_data_corrupted:
    ourmemsetzero(samps, transform_size * 4);
    return;
  }

  // read the first two
  {
    // 29 bits each (FXPBITS)
    U64 fxp = read_up_to_56_bits(buffer, bit_position);
    bit_position += FXPBITS;
    fxp &= ((1 << FXPBITS) - 1);

    samps[0] = fxptof((U32)fxp);

    fxp = read_up_to_56_bits(buffer, bit_position);
    bit_position += FXPBITS;
    fxp &= ((1 << FXPBITS) - 1);

    samps[1] = fxptof((U32)fxp);
  }

  // unquantize the thresholds
  threshold[0] = 0;
  for (; i < num_bands; i++)
  {
    // each threshold lookup is 7 bits
    U32 j = read_up_to_24_bits(buffer, bit_position);
    bit_position += 7;
    j &= 0x7f;

    if (j > 95)   // sizeof(Undecibel)/sizeof(*Undecibel) - 1
      j = 95;     // Only look so far into the table, the rest is inaudible range
    threshold[i + 1] = bink_Undecibel_table[j];//(F32) Undecibel( ( (F32) (S32) j ) * 0.664F );
  }

  // decode the rle runs
  b = 0;
  for (i = 2; i < transform_size; )
  {
    int bitlen;
    size_t tmp;
    size_t end;

    if (validate_chunked_bit_read<uint32_t>(bit_position, end_bit_position, 9) == false)
      goto read_chan_data_corrupted;

    // 9 bits contains all our metadata
    tmp = read_up_to_24_bits(buffer, bit_position);
    if (tmp & 1) // is rle?
    {
      // next 4 bits are index in to rle run lengths
      // 1 flag + 4 rle + 4 bitlen
      bit_position += 9;
      end = ((tmp >> 1) & 15);
      tmp >>= 5;
      end = i + (bink_rlelens_snd[end] * VQLENGTH);
    }
    else
    {
      // a single run
      // 1 flag + 4 bitlen
      bit_position += 5;
      tmp >>= 1;
      end = i + VQLENGTH;
    }

    if (end > transform_size)
      end = transform_size;

    // remaining 4 bits are bitlen
    bitlen = tmp & 15;
    if (bitlen == 0)
    {
      ourmemsetzero(samps + i, (end - i) * 4);
      i = end;
    }
    else
    {
      size_t coeff_bits_needed = (bitlen * (end - i));
      size_t sign_bit_position = bit_position + coeff_bits_needed;

      {
        //
        // Validation
        // 
        // The farthest bit we can actually use, assuming all non-zero coeffs,
        // is sign_bit_position + run_length (minus 1).
        // 
        // However, for SSSE3 we read 128 bit chunks for coeffs, and 32 bit chunks
        // for signs. So a short run of 8 (very common) could mean the coeff vector
        // read is actually farther than the sign read. So we have to check both.
        //
        if (validate_chunked_bit_read<uint32_t>(sign_bit_position, end_bit_position, (end - i)) == false)
          goto read_chan_data_corrupted;

#if defined(__RADX64__) && defined(RAD_USES_SSSE3)
#ifndef FORCE_SCALAR
        if (CPU_can_use(CPU_SSSE3))
        {
          if (validate_chunked_bit_read<Vec128>(bit_position, end_bit_position, coeff_bits_needed) == false)
            goto read_chan_data_corrupted;
          bit_position = decode_pfxsum(samps + i, buffer, bit_position, sign_bit_position, end_bit_position, end - i, bitlen);
        }
        else
#endif
        {
          if (validate_chunked_bit_read<uint64_t>(bit_position, end_bit_position, coeff_bits_needed) == false)
            goto read_chan_data_corrupted;
          bit_position = decode_scalar(samps + i, buffer, bit_position, sign_bit_position, end - i, bitlen);
        }
#else
        if (validate_chunked_bit_read<uint64_t>(bit_position, end_bit_position, coeff_bits_needed) == false)
          goto read_chan_data_corrupted;
        bit_position = decode_scalar(samps + i, buffer, bit_position, sign_bit_position, end - i, bitlen);
#endif

        // all of the decoded values need to be scaled by the threshold,
        // which changes depending on the current band.
        while (i < end)
        {
          size_t bandend;                                        

          // figure out which band we're in
          while (i >= (bands[b] * 2))
            ++b;

          bandend = bands[b] * 2;
          if (end < bandend)
            bandend = end;

#ifdef __RADX64__
          // We can assume SSE2
          {
            size_t thresh_count = bandend - i;
            size_t simd_runs_end = thresh_count & ~0x7;
            __m128 threshold_vec = _mm_set_ps1(threshold[b]);
            
            size_t t = 0;
            for (t = 0; t < simd_runs_end; t += 8)
            {
              __m128 svec1 = _mm_loadu_ps(samps + i + t);
              __m128 svec2 = _mm_loadu_ps(samps + i + t + 4);
              svec1 = _mm_mul_ps(svec1, threshold_vec);
              svec2 = _mm_mul_ps(svec2, threshold_vec);
              _mm_storeu_ps(samps + i + t, svec1);
              _mm_storeu_ps(samps + i + t + 4, svec2);
            }

            // this is here to keep some compilers from autogenerating
            // the 8x unroll if it supports it (vs2010 doesnt, 2019 does).
            // since cdep builds with vs2010, we need to manually do the one
            // above.
            if ((thresh_count - t) < 8)
            {
              for (; t < thresh_count; t++)
                samps[i + t] = threshold[b] * samps[i + t];
            }
          }
#elif __RADARM__
          {
            size_t thresh_count = bandend - i;
            size_t simd_runs_end = thresh_count & ~0x7;
            float32x4_t threshold_vec = vdupq_n_f32(threshold[b]);
            
            size_t t = 0;
            for (t = 0; t < simd_runs_end; t += 8)
            {
              float32x4_t one = vld1q_f32(samps + i + t);
              float32x4_t two = vld1q_f32(samps + i + t + 4);
              one = vmulq_f32(one, threshold_vec);
              two = vmulq_f32(two, threshold_vec);
              vst1q_f32(samps + i + t, one);
              vst1q_f32(samps + i + t + 4, two);
            }

            for (; t < thresh_count; t++)
              samps[i + t] = threshold[b] * samps[i + t];
          }
#else
          for (size_t t=i; t < bandend; t++)
            samps[t] = threshold[b] * samps[t];
#endif

          i = bandend;
        } // end rle
      } // artificial scope
    } // end if not zeros
  } // end transform size

  // Convert back from base + offset to varbits
  buffer += 8 * (bit_position / 64);
  bit_position &= 63;

  vbp->bits = *(U64*)buffer;
  vbp->cur = (U64*)buffer + 1;
  vbp->bitlen = (U32)(64 - bit_position);
  vbp->bits >>= bit_position;
}

#else // __RAD64__


static RADINLINE void short_to_float_4(float* output, short* input)
{
    size_t i = 0;
    for (i = 0; i < 4; i++)
        output[i] = (float)input[i];
}

template <int bitlen>
static RADINLINE size_t decode_coeffs_runlength_4(float* results, const uint8_t* buffer, size_t coeff_position, size_t sign_position, size_t count)
{
    // 4 ceoffs fits in the 24 bits we can get - so we grab chunks of 4
    size_t run_count = count & ~3;
    size_t run = 0;

    for (run = 0; run < run_count; run+=4)
    {
        S16 coeffs[4];
        unsigned char current_sign_bit = 0;
        U32 coeff_bits = read_up_to_24_bits(buffer, coeff_position);
        coeff_position += 4 * bitlen;

        U32 sign_bits = read_up_to_24_bits(buffer, sign_position);

        decode_coeff<bitlen, 0>(current_sign_bit, coeff_bits, sign_bits, coeffs);
        decode_coeff<bitlen, 1>(current_sign_bit, coeff_bits, sign_bits, coeffs);
        decode_coeff<bitlen, 2>(current_sign_bit, coeff_bits, sign_bits, coeffs);
        decode_coeff<bitlen, 3>(current_sign_bit, coeff_bits, sign_bits, coeffs);

        sign_position += current_sign_bit;                

        short_to_float_4(results + run, coeffs);
    }

    return decode_coeff_remnants<bitlen>(run, results, buffer, coeff_position, sign_position, count);
}


static U32 RADINLINE decode_scalar(float* results, const uint8_t* buffer, size_t coeff_position, size_t sign_position, size_t count, int bitlen)
{
    switch (bitlen)
    {
    case 1: /* 1*4 = 4 <= 24 */  return (U32)decode_coeffs_runlength_4<1>(results, buffer, coeff_position, sign_position, count);
    case 2: /* 2*4 = 8 <= 24 */ return (U32)decode_coeffs_runlength_4<2>(results, buffer, coeff_position, sign_position, count);
    case 3: /* 3*4 = 12 <= 24 */ return (U32)decode_coeffs_runlength_4<3>(results, buffer, coeff_position, sign_position, count);
    case 4: /* 4*4 = 16 <= 24 */ return (U32)decode_coeffs_runlength_4<4>(results, buffer, coeff_position, sign_position, count);
    case 5: /* 5*4 = 20 <= 24 */ return (U32)decode_coeffs_runlength_4<5>(results, buffer, coeff_position, sign_position, count);
    case 6: /* 6*4 = 24 <= 24 */ return (U32)decode_coeffs_runlength_4<6>(results, buffer, coeff_position, sign_position, count);
    case 7: return (U32)decode_coeff_remnants<7>(0, results, buffer, coeff_position, sign_position, count);
    case 8: return (U32)decode_coeff_remnants<8>(0, results, buffer, coeff_position, sign_position, count);
    case 9: return (U32)decode_coeff_remnants<9>(0, results, buffer, coeff_position, sign_position, count);
    case 10: return (U32)decode_coeff_remnants<10>(0, results, buffer, coeff_position, sign_position, count);
    case 11: return (U32)decode_coeff_remnants<11>(0, results, buffer, coeff_position, sign_position, count);
    case 12: return (U32)decode_coeff_remnants<12>(0, results, buffer, coeff_position, sign_position, count);
    case 13: return (U32)decode_coeff_remnants<13>(0, results, buffer, coeff_position, sign_position, count);
    case 14: return (U32)decode_coeff_remnants<14>(0, results, buffer, coeff_position, sign_position, count);
    case 15: return (U32)decode_coeff_remnants<15>(0, results, buffer, coeff_position, sign_position, count);
    }
    // invalid data.
    return (U32)sign_position;
}


static RADINLINE U32 read_up_to_32_bits(const U8* buffer, size_t bit_position)
{
    uint32_t x1;
    uint32_t x2;
    memcpy(&x1, buffer + (bit_position >> 3), sizeof(x1));
    if ((bit_position & 7) == 0)
      return x1;

    memcpy(&x2, buffer + (bit_position >> 3) + 4, sizeof(x2));
    uint32_t x1_aligned = x1 >> (bit_position & 7);
    uint32_t x2_aligned = x2 << (32 - (bit_position & 7));
    return x1_aligned | x2_aligned;
}

static void read_channel_data_2( F32 * samps,
                               U32 transform_size,
                               U32 num_bands,
                               BINKVARBITS * vbp, 
                               U32 * bands,
                               void * padded_in_data_end ) 
{
  size_t i = 0;
  size_t b;
  F32 threshold[TOTBANDS + 2];

  // convert from varbits to base + offset
  U8* buffer = ((U8*)vbp->cur) - 4;
  size_t bit_position = 32 - vbp->bitlen;
  U32 end_bit_position = (U32)(((U8*)padded_in_data_end - buffer) * 8);

  // validate we can get to the transform
  U32 bits_needed = num_bands * 8 + FXPBITS * 2;
  if (validate_chunked_bit_read<uint32_t>(bit_position, end_bit_position, bits_needed) == false)
  {
    // corrupted data.
read_chan_data_corrupted:
    ourmemsetzero(samps, transform_size * 4);
    return;
  }

  // read the first two
  {
    // 29 bits each (FXPBITS)
    U32 fxp = read_up_to_32_bits(buffer, bit_position);
    bit_position += FXPBITS;
    fxp &= ((1 << FXPBITS) - 1);

    samps[0] = fxptof((U32)fxp);

    fxp = read_up_to_32_bits(buffer, bit_position);
    bit_position += FXPBITS;
    fxp &= ((1 << FXPBITS) - 1);

    samps[1] = fxptof((U32)fxp);
  }

  // unquantize the thresholds
  threshold[0] = 0;
  for (; i < num_bands; i++)
  {
    // each threshold lookup is 7 bits
    U32 j = read_up_to_24_bits(buffer, bit_position);
    bit_position += 7;
    j &= 0x7f;

    if (j > 95)   // sizeof(Undecibel)/sizeof(*Undecibel) - 1
      j = 95;     // Only look so far into the table, the rest is inaudible range
    threshold[i + 1] = bink_Undecibel_table[j];//(F32) Undecibel( ( (F32) (S32) j ) * 0.664F );
  }

  // decode the rle runs
  b = 0;
  for (i = 2; i < transform_size; )
  {
    int bitlen;
    size_t tmp;
    size_t end;

    if (validate_chunked_bit_read<uint32_t>(bit_position, end_bit_position, 9) == false)
      goto read_chan_data_corrupted;

    // 9 bits contains all our metadata
    tmp = read_up_to_24_bits(buffer, bit_position);
    if (tmp & 1) // is rle?
    {
      // next 4 bits are index in to rle run lengths
      // 1 flag + 4 rle + 4 bitlen
      bit_position += 9;
      end = ((tmp >> 1) & 15);
      tmp >>= 5;
      end = i + (bink_rlelens_snd[end] * VQLENGTH);
    }
    else
    {
      // a single run
      // 1 flag + 4 bitlen
      bit_position += 5;
      tmp >>= 1;
      end = i + VQLENGTH;
    }

    if (end > transform_size)
      end = transform_size;

    // remaining 4 bits are bitlen
    bitlen = tmp & 15;
    if (bitlen == 0)
    {
      ourmemsetzero(samps + i, (end - i) * 4);
      i = end;
    }
    else
    {
      size_t coeff_bits_needed = (bitlen * (end - i));
      size_t sign_bit_position = bit_position + coeff_bits_needed;

      {
        //
        // Validation
        // 
        // The farthest bit we can actually use, assuming all non-zero coeffs,
        // is sign_bit_position + run_length (minus 1).
        // 
        // However, for SSSE3 we read 128 bit chunks for coeffs, and 32 bit chunks
        // for signs. So a short run of 8 (very common) could mean the coeff vector
        // read is actually farther than the sign read. So we have to check both.
        //
        if (validate_chunked_bit_read<uint32_t>(sign_bit_position, end_bit_position, (end - i)) == false)
          goto read_chan_data_corrupted;

        {
          if (validate_chunked_bit_read<uint32_t>(bit_position, end_bit_position, coeff_bits_needed) == false)
            goto read_chan_data_corrupted;
          bit_position = decode_scalar(samps + i, buffer, bit_position, sign_bit_position, end - i, bitlen);
        }

        // all of the decoded values need to be scaled by the threshold,
        // which changes depending on the current band.
        while (i < end)
        {
          size_t bandend;                                        

          // figure out which band we're in
          while (i >= (bands[b] * 2))
            ++b;

          bandend = bands[b] * 2;
          if (end < bandend)
            bandend = end;

          for (size_t t=i; t < bandend; t++)
            samps[t] = threshold[b] * samps[t];

          i = bandend;
        } // end rle
      } // artificial scope
    } // end if not zeros
  } // end transform size

  // Convert back from base + offset to varbits
  buffer += 4 * (bit_position / 32);
  bit_position &= 31;

  vbp->bits = *(U32*)buffer;
  vbp->cur = (U32*)buffer + 1;
  vbp->bitlen = (U32)(32 - bit_position);
  vbp->bits >>= bit_position;
}


#endif // __RAD64__

static void read_channel_data(F32* samps,
    U32 transform_size,
    U32 num_bands,
    BINKVARBITS* vbp,
    U32* bands,
    void* padded_in_data_end)
{
  // bink audio 1 version - signs after coeffs
  U32 i, b;
  F32 threshold[ TOTBANDS + 2 ];
  void * init;

  BINKBITSLOCALS( lvb );

  VarBitsCopyToBinkBits( lvb, *vbp );
  init = lvbcur;

   // read the first two
  BinkBitsGet( i, U32, lvb, FXPBITS, ((1<<FXPBITS)-1) );
  samps[ 0 ] = fxptof( i );
  BinkBitsGet( i, U32, lvb, FXPBITS, ((1<<FXPBITS)-1) );
  samps[ 1 ] = fxptof( i );

  // unquantize the thresholds
  threshold[ 0 ] = 0;
  for ( i = 0 ; i < num_bands ; i++ )
  {
    U32 j;

    BinkBitsGet( j, U32, lvb, 8, ((1<<8)-1) );
    if (j > 95)   // sizeof(Undecibel)/sizeof(*Undecibel) - 1
      j = 95;     // Only look so far into the table, the rest is inaudible range
    threshold[ i + 1 ] = bink_Undecibel_table[ j ];//(F32) Undecibel( ( (F32) (S32) j ) * 0.664F );
  }

  b = 0;
  for( i = 2 ; i < transform_size ; )
  {
    U32 bitlen;
    U32 tmp;
    U32 end;
    
    BinkBitsPeek( tmp, U32, lvb, 1+4+4 );
    if ( tmp & 1 )
    {
      BinkBitsUse( lvb, 9 );
      end = ( ( tmp >> 1 ) & 15 );
      tmp >>= 5;
      end = i + ( bink_rlelens_snd[ end ] * VQLENGTH );
    }
    else
    {
      BinkBitsUse( lvb, 5 );
      tmp >>= 1;
      end = i + VQLENGTH;
    }
 
    if ( end > transform_size )
      end = transform_size;

    bitlen = tmp & 15;
    if ( bitlen == 0 )
    {
     clear:
      ourmemsetzero( samps+i, (end-i)*4 );
      i = end;
    }
    else
    {
      U32 bitmask = ( 1 << bitlen ) - 1;

      {
        U32 bp1 = bitlen + 1;

        if ( ( ((U8*)lvbcur) + (((( end - i )*bp1)+9)/8) ) > (U8*)padded_in_data_end )
        {
          // check to see if the maximum bit use per count (bp1 per sample) plus the next 
          // we are going to read passed out end point, which means a corruption
          //   has occurred - zap this range and get out

          end = transform_size; // clear to end, so we just fall out of the loop
          lvbcur = init; // reset read position  
          goto clear;
        }

        while ( i < end )
        {
          U32 bandend;
          F32 q[2];
          F32 * s, * s_end;
          //U32 v;
          U32 used;
          S32 val;

          // figure out which band we're in
          while ( i >= ( bands[ b ] * 2 ) )
            ++b;

          // decode either up to "end" or to the end of this band,
          // whichever is earlier.
          bandend = bands[ b ] * 2;
          if ( end < bandend )
            bandend = end;

          s = samps + i;
          s_end = samps + bandend;

          q[0] = threshold[b];
          q[1] = -threshold[b];

          // do four at a times for smaller bitlens
          if ( ( ( i + 4 ) <= bandend ) && ( bp1 <= (MAX_AT_LEAST_BITS/4) ) )   // div 4 because we unroll 4 reads
          {
            U32 bp14 = bp1*4;

            s_end -= 4;
            do
            {
              BinkBitsAtLeastStart( lvb, bp14 );

              val = ((S32)BinkBitsInAtLeastPeek(lvb)) & bitmask;
              BinkBitsInAtLeastUse( lvb, bitlen );
              s[0] = ((F32)val)*q[BinkBitsInAtLeastPeek(lvb)&1];
              used = val != 0; 
              BinkBitsInAtLeastUse( lvb, used );

              val = ((S32)BinkBitsInAtLeastPeek(lvb)) & bitmask;
              BinkBitsInAtLeastUse( lvb, bitlen );
              s[1] = ((F32)val)*q[BinkBitsInAtLeastPeek(lvb)&1];
              used = val != 0; 
              BinkBitsInAtLeastUse( lvb, used );

              val = ((S32)BinkBitsInAtLeastPeek(lvb)) & bitmask;
              BinkBitsInAtLeastUse( lvb, bitlen );
              s[2] = ((F32)val)*q[BinkBitsInAtLeastPeek(lvb)&1];
              used = val != 0; 
              BinkBitsInAtLeastUse( lvb, used );

              val = ((S32)BinkBitsInAtLeastPeek(lvb)) & bitmask;
              BinkBitsInAtLeastUse( lvb, bitlen );
              s[3] = ((F32)val)*q[BinkBitsInAtLeastPeek(lvb)&1];
              used = val != 0; 
              BinkBitsInAtLeastUse( lvb, used );

              BinkBitsAtLeastEnd( lvb );
              s += 4;
            } while ( s <= s_end );

            s_end += 4;
            i = (U32)( s_end - s );
            if ( i >= bandend )
              continue;
          }

          // now the remenants (or all of them, if the bitlen is big)        
          RADASSUME( ( s_end - s ) < 4 );  // hopefully no unroll, sigh
          while ( s < s_end ) 
          {
            BinkBitsAtLeastStart( lvb, bp1 );

            val = ((S32)BinkBitsInAtLeastPeek(lvb)) & bitmask;
            BinkBitsInAtLeastUse( lvb, bitlen );
            *s = ((F32)val)*q[BinkBitsInAtLeastPeek(lvb)&1];
            used = val != 0;
            BinkBitsInAtLeastUse( lvb, used );
            BinkBitsAtLeastEnd(lvb);
            ++s;
          }

          i = bandend;
        }
      }
    }
  }
  BinkBitsCopyToVarBits( *vbp, lvb );
}

// we return the pointer in out_ptr, if we have room, otherwise we return 0, and you have to alloca a buffer
static int have_room( void ** out_ptr, BINKAC_OUT_RINGBUF * out, U32 need )
{
  U32 have;

  have = (U32)( ((U8*)out->outend) - ((U8*)out->outptr) );
  if ( have > out->outlen ) have = out->outlen;
      
  // start by assuming we fit
  *out_ptr = out->outptr;

  // do we fit at the current pointer (at outptr)?
  if ( need > have )
  {
    // nope, well, would we fit at the front of the circular buffer?
    have = out->outlen - have;
    if ( need > have )
      return 0; // nope!

    *out_ptr = out->outstart; // use front of buffer
  }

  return 1;
}

static void update_ring( BINKAC_OUT_RINGBUF * out, S16 const * from, U32 bytes )
{
  U32 left;
  
  out->decoded_bytes += bytes;

  // if we have to eat front data, do it here
  if ( out->eatfirst )
  {
    if ( out->eatfirst > bytes )
    {
      out->eatfirst -= bytes;
      bytes = 0;
      out->outlen = 0;
      return;
    }
    else
    {
      bytes -= out->eatfirst;
      from = (S16*)( ((U8*)from) + out->eatfirst );
      out->eatfirst = 0;
    }
  }

  // limit to how much left
  if ( bytes > out->outlen ) bytes = out->outlen;
  out->outlen = bytes;
  
  // how much to the end of the ring buffer?
  left = (U32) ( ((U8*)out->outend) - ((U8*)out->outptr) );

  // limit to how many bytes that we have
  if ( left > bytes ) left = bytes;
  
  // move the data, if we have to
  if ( from != out->outptr ) 
  {
    ourmemcpy( (S16*)out->outptr, from, left );
    from = (S16*)( ((U8*)from) + left );
    out->outptr = (S16*)( ((U8*)out->outptr) + left );
    bytes -= left;
    if ( bytes )
    {
      out->outptr = out->outstart;
      ourmemcpy( out->outptr, from, bytes );
      out->outptr = (S16*)( ((U8*)out->outptr) + bytes );
    }
  }
  else
  {
    out->outptr = (S16*)( ((U8*)out->outptr) + bytes );
  }
}

static void linear_inverse_transform_to_s16( U32 flags, S16 * buf, F32 * decoded_coeffs, U32 transform_size, F32 transform_size_root, S16 * overlap, U32 window_size_in_bytes, U32 window_shift )
{
  F32 * f;

  f = (F32*) alloca( ( sizeof(F32)*transform_size ) + 64 ); // plus 64 for align
  f = (F32*) ( ( ( (UINTa)f ) + 63 ) & ~63 ); // align it

  // do the inverse transform
  {
    radfft_idct_to_S16( buf, transform_size_root, f, decoded_coeffs, transform_size );
  }

  // fade in the front
  if ( window_shift ) CallCrossFade( buf, overlap, window_size_in_bytes, window_shift );

  // Store end of buffer
  if ( window_size_in_bytes ) ourmemcpy( overlap, (U8*)(buf + transform_size) - window_size_in_bytes, window_size_in_bytes );
}


static void inverse_transform_to_s16( U32 flags, BINKAC_OUT_RINGBUF * out, F32 * decoded_coeffs, U32 transform_size, F32 transform_size_root, S16 * overlap, U32 window_size_in_bytes, U32 window_shift )
{
  S16 * buf;
  U32 need;

  need = transform_size * sizeof(S16);

  if ( !have_room( (void**)&buf, out, need ) )
  {
    buf = (S16*)alloca( need + 16 );
    buf = (S16*) ( ( ( (UINTa)buf ) + 15 ) & ~15 ); // align it
  }

  linear_inverse_transform_to_s16( flags, buf, decoded_coeffs, transform_size, transform_size_root, overlap, window_size_in_bytes, window_shift );

  update_ring( out, buf, ( sizeof(S16) * transform_size ) - window_size_in_bytes ); 
}


static void inverse_transform_to_s16_stereo( U32 flags, BINKAC_OUT_RINGBUF * out, S16 * buf, F32 * decoded_coeffs, U32 transform_size, F32 transform_size_root, S16 * overlap, U32 window_size_in_bytes, U32 window_shift )
{
  F32 * f;
  S16 * left;

  left = buf + transform_size;

  f = (F32*) alloca( ( sizeof(F32)*transform_size ) + 64 ); // plus 64 for align
  f = (F32*) ( ( ( (UINTa)f ) + 63 ) & ~63 ); // align it

  // do the inverse transform
  radfft_idct_to_S16_stereo_interleave( buf, left, transform_size_root, f, decoded_coeffs, transform_size );

  rrassert( window_size_in_bytes != 0 );
  
  // fade in the front
  if ( window_shift ) CallCrossFade( buf, overlap, window_size_in_bytes, window_shift );

  // Store end of buffer
  if ( window_size_in_bytes ) ourmemcpy( overlap, ((U8*)(buf + (transform_size*2))) - window_size_in_bytes, window_size_in_bytes ); 

  update_ring( out, buf, ( sizeof(S16) * transform_size * 2 ) - window_size_in_bytes ); 
}

// decode the data into an output buffer and return amount read from input
static void decode_frame( U32 transform_size,
                          F32 transform_size_root,
                          U32 chans,
                          U32 flags,
                          BINKAC_OUT_RINGBUF * output,
                          BINKAC_IN * input,
                          U32 num_bands,
                          U32 * bands,
                          U32 window_size_in_bytes,
                          U32 window_shift,
                          S16 * overlap )                   
{
  BINKVARBITS vb;
  F32 * decoded_coeffs;
  void * padded_in_data_end;

  decoded_coeffs = (F32*) alloca( ( sizeof(F32) * transform_size ) + 64 + 32 ); // plus 64 for align, 32 for zero overwrite
  decoded_coeffs = (F32*) ( ( ( (UINTa)decoded_coeffs ) + 63 ) & ~63 ); // align it

  padded_in_data_end = ( (U8*)input->inend ) + BINKACD_EXTRA_INPUT_SPACE;
  BinkVarBitsOpen( vb, input->inptr );

  {
    BinkVarBitsUse( vb, 2 );
  }

  if ( chans == 1 )
  {
    if (flags & BINKAC20)
      read_channel_data_2(decoded_coeffs, transform_size, num_bands, &vb, bands, padded_in_data_end);
    else
      read_channel_data( decoded_coeffs, transform_size, num_bands, &vb, bands, padded_in_data_end );
    inverse_transform_to_s16( flags, output, decoded_coeffs, transform_size, transform_size_root, overlap, window_size_in_bytes, window_shift );
  }
  else
  {
    if ( flags & BINKACNODEINTERLACE )
    {
      U32 eatfirst;

      if ( window_shift ) --window_shift;
      window_size_in_bytes >>= 1;

      eatfirst = output->eatfirst;
      output->eatfirst >>= 1; // only eat half for left channel

      if (flags & BINKAC20)
          read_channel_data_2(decoded_coeffs, transform_size, num_bands, &vb, bands, padded_in_data_end);
      else
        read_channel_data( decoded_coeffs, transform_size, num_bands, &vb, bands, padded_in_data_end );
      inverse_transform_to_s16( flags, output, decoded_coeffs, transform_size, transform_size_root, overlap, window_size_in_bytes, window_shift );

      output->eatfirst = eatfirst - ( (eatfirst>>1)-output->eatfirst ); // shrink the eatfirst by the amount that the left channel consumed

      if (flags & BINKAC20)
          read_channel_data_2(decoded_coeffs, transform_size, num_bands, &vb, bands, padded_in_data_end);
      else
        read_channel_data( decoded_coeffs, transform_size, num_bands, &vb, bands, padded_in_data_end );
      inverse_transform_to_s16( flags, output, decoded_coeffs, transform_size, transform_size_root, (S16*)(((U8*)overlap)+window_size_in_bytes), window_size_in_bytes, window_shift );
    }
    else
    {
      U32 need;
      S16 * buf;

      need = transform_size * 2 * sizeof(S16); // 2 for stereo

      // make a ring buf struct just for the left output channels (we interlace in the second inverse_transform)
      if ( !have_room( (void**)&buf, output, need ) )
      {
        // use temp stack buffer
        buf = (S16*)alloca( need + 16 );
        buf = (S16*) ( ( ( (UINTa)buf ) + 15 ) & ~15 ); // align it
      }

      if (flags & BINKAC20)
        read_channel_data_2(decoded_coeffs, transform_size, num_bands, &vb, bands, padded_in_data_end);
      else
        read_channel_data( decoded_coeffs, transform_size, num_bands, &vb, bands, padded_in_data_end );
      linear_inverse_transform_to_s16( flags, buf + transform_size, decoded_coeffs, transform_size, transform_size_root, 0, 0, 0 ); 

      if (flags & BINKAC20)
          read_channel_data_2(decoded_coeffs, transform_size, num_bands, &vb, bands, padded_in_data_end);
      else
        read_channel_data( decoded_coeffs, transform_size, num_bands, &vb, bands, padded_in_data_end );
      inverse_transform_to_s16_stereo( flags, output, buf, decoded_coeffs, transform_size, transform_size_root, overlap, window_size_in_bytes, window_shift );
    }
  }

  // Store the results back in data
  input->inptr = ( (U8*) input->inptr ) + BinkVarBitsSizeBytesRoundedToU32( vb, input->inptr );
}

#define roundup( val ) ( ( ( val ) + 15 ) & ~15 )

RADDEFFUNC U32 RADLINK BinkAudioDecompressMemory( U32 rate,
                                                  U32 chans,
                                                  U32 flags )
{
  U32 ptr_end;
  U32 work_end;
  U32 overlap_end;
  U32 transform_size, buffer_size;

  if ( rate >= 44100 )
    transform_size = 2048;
  else if ( rate >= 22050 )
    transform_size = 1024;
  else
    transform_size = 512;

  // in bytes
  buffer_size = transform_size * chans * 2;
  ptr_end = roundup( sizeof( BINKAUDIODECOMP ) );
  overlap_end = roundup( ptr_end + ( buffer_size / WINDOWRATIO ) );
  work_end = overlap_end;

  return work_end;
}

RADDEFFUNC void RADLINK BinkAudioDecompressResetStartFrame(void* mem)
{
  HBINKAUDIODECOMP ba = (HBINKAUDIODECOMP)mem;
  // if we stop and want to restart the decompression at a new spot, we need to clear 
  // the start frame - otherwise we blend in the last frame.
  if (ba)
    ba->start_frame = 1;
}

#define SQRT2 1.41421356237309504880f
static F32 tsr4096 = 2.0f / 64.0f;
static F32 tsr2048 = 2.0f / (32.0f*SQRT2);
static F32 tsr1024 = 2.0f / 32.0f;
static F32 tsr512  = 2.0f / (16.0f*SQRT2);

RADDEFFUNC U32 RADLINK BinkAudioDecompressOpen( void * mem,
                                                             U32 rate,
                                                             U32 chans,
                                                             U32 flags )
{
  U32 i;
  U32 transform_size, transform_size_half, buffer_size;
  F32 transform_size_root, transform_size_root_big;
  U32 num_bands;
  S32 nyq;
  HBINKAUDIODECOMP ba;

  CPU_check( 0, 0 );

  if ( rate >= 44100 )
  {
    transform_size = 2048;
    transform_size_root = tsr2048;
    transform_size_root_big = tsr4096;
  }
  else if ( rate >= 22050 )
  {
    transform_size = 1024;
    transform_size_root = tsr1024;
    transform_size_root_big = tsr2048;
  }
  else
  {
    transform_size = 512;
    transform_size_root = tsr512;
    transform_size_root_big = tsr1024;
  }

  // in bytes
  buffer_size = transform_size * chans * 2;

  if ( transform_size > MAX_TRANSFORM )
    return 0;

  transform_size_half = transform_size / 2;
  nyq = ( rate + 1 ) / 2;

  // calculate the number of bands we'll use
  for( i = 0 ; i < TOTBANDS ; i++ )
  {
    if ( bink_bandtopfreq[ i ] >= (U32) nyq )
      break;
  }
  num_bands = i;

  // allocate our memory
  {
    U32 ptr_end;
    U32 work_end;
    U32 overlap_end;

    ptr_end = roundup( sizeof( BINKAUDIODECOMP ) );
    overlap_end = roundup( ptr_end + ( buffer_size / WINDOWRATIO ) );

    work_end = overlap_end;

    ba = (HBINKAUDIODECOMP) mem;

    ourmemsetzero( ba, sizeof( BINKAUDIODECOMP ) );

    ba->overlap = (S16*) ( ( (U8*) ba ) + ptr_end );
    ba->size = work_end;

    radfft_init();
  }

  ba->flags = flags;

  if ( chans == 1 )
    ba->flags &= ~BINKACNODEINTERLACE;

  ba->chans = chans;
  ba->num_bands = num_bands;
  ba->transform_size = transform_size;

  ba->buffer_size = buffer_size;
  ba->window_size_in_bytes = buffer_size / WINDOWRATIO;
  ba->window_shift = 0;

  switch (ba->window_size_in_bytes) // shift amount to divide by number of samples in the window.
  {
    case 512:
        ba->window_shift = 8; break;
    case 256:
        ba->window_shift = 7; break;
    case 128:
        ba->window_shift = 6; break;
    case 64:
        ba->window_shift = 5; break;
  
    default:
        RR_BREAK();
  }

  ba->transform_size_root = transform_size_root;

  // calculate the band ranges
  for( i = 0 ; i < num_bands ; i++ )
  {
    ba->bands[ i ] = ( bink_bandtopfreq[ i ] * transform_size_half ) / nyq;
    if ( ba->bands[ i ] == 0 )
      ba->bands[ i ] = 1;
  }
  ba->bands[ i ] = transform_size_half;

  ba->start_frame = 1;

  return 1;
}

RADDEFFUNC void RADLINK BinkAudioDecompress( void* mem,
                                             BINKAC_OUT_RINGBUF * output,
                                             BINKAC_IN * input )
{
  HBINKAUDIODECOMP ba = (HBINKAUDIODECOMP)mem;
  output->decoded_bytes = 0;

  // wrap pointer, if outer ringbuffer code didn't
  if ( output->outptr == output->outend ) output->outptr = output->outstart;
  
  decode_frame( ba->transform_size,
                ba->transform_size_root,
                ba->chans,
                ba->flags,
                output,
                input,
                ba->num_bands,
                ba->bands,
                ba->window_size_in_bytes,
                ( ba->start_frame ) ? 0 : ba->window_shift,
                ba->overlap );
  
  // reset after decoding each frame 
  ba->start_frame = 0;
}
 
RADDEFFUNC U32 RADLINK BinkAudioDecompressOutputSize(void* mem)
{
  HBINKAUDIODECOMP ba = (HBINKAUDIODECOMP)mem;
  return ba->buffer_size;
}

//#define TEST_READ_CHAN_DATA
#ifdef TEST_READ_CHAN_DATA

#include <stdio.h>
#include "ticks.h"
#include <assert.h>
#include <vector>
using namespace std;


static uint32_t read_bits_ref(const uint8_t * buffer, size_t buffer_len, size_t bitpos, int width)
{
    uint32_t bytes = 0;

    size_t begin = bitpos >> 3;
    size_t end = (bitpos + width + 7) >> 3;

    // little-endian read (clamped)
    int shift = 0;
    for (size_t pos = begin; pos < end; ++pos, shift += 8)
    {
        assert(pos < buffer_len);
        bytes |= buffer[pos] << shift;
    }

    uint32_t aligned = bytes >> (bitpos & 7);
    uint32_t mask = (1u << width) - 1;
    return aligned & mask;
}

// reads a fully qualified bink audio 2 data stream for 1 channel.
static bool read_channel_data_ref(float * results, const uint8_t * buffer, size_t buffer_len, int total_entries, size_t initial_bitpos)
{
    // read initial values - test stream sets these to 0
    size_t bitpos = initial_bitpos;
    results[0] = fxptof(read_bits_ref(buffer, buffer_len, bitpos, FXPBITS));
    bitpos += FXPBITS;
    results[1] = fxptof(read_bits_ref(buffer, buffer_len, bitpos, FXPBITS));
    bitpos += FXPBITS;

    // test stream has 1 threshold value, which is 0, corresponding to a 
    // 1.0f multiplier.
    U32 threshold_index = read_bits_ref(buffer, buffer_len, bitpos, 7);
    assert(threshold_index == 0);
    float threshold = bink_Undecibel_table[threshold_index];
    bitpos += 7;
    
    for (int i = 2; i < total_entries;)
    {
        // decode the run - must be a single run entry
        if (read_bits_ref(buffer, buffer_len, bitpos, 1) != 0)
        {
            assert(0);
            return false;
        }
        bitpos++;

        uint32_t bitlen = read_bits_ref(buffer, buffer_len, bitpos, 4);
        bitpos += 4;

        int runlength = 8;

        size_t sign_pos = bitpos + runlength * bitlen;
        for (int r = 0; r < runlength; r++, i++)
        {
            int32_t coeff = read_bits_ref(buffer, buffer_len, bitpos, bitlen);
            bitpos += bitlen;

            if (coeff)
            {
                if (read_bits_ref(buffer, buffer_len, sign_pos, 1))
                    coeff = -coeff;
                sign_pos++;
            }

            results[i] = (float)coeff * threshold;
        }

        bitpos = sign_pos;
    }

    return true;
}

static void encode_bits(U8* bitstream, size_t bitstream_len, size_t bitpos, unsigned int value, int bitlen)
{
    size_t bitend = bitpos + bitlen;
    size_t bytepos = bitpos >> 3;
    size_t byteend = (bitend + 7) >> 3;
    size_t bytecount = byteend - bytepos;

    uint64_t code_shifted = value << (bitpos & 7);
    for (size_t j = 0; j < bytecount; ++j)
    {
        assert(bytepos + j < bitstream_len);
        bitstream[bytepos + j] |= code_shifted >> (j * 8);
    }
}

static void encode_value(U8* bitstream, size_t bitstream_len, size_t bitpos, size_t &sign_bitpos, int value, int bitlen)
{
    int magnitude = abs(value);
    assert(magnitude < (1 << bitlen));

    size_t bitend = bitpos + bitlen;
    size_t bytepos = bitpos >> 3;
    size_t byteend = (bitend + 7) >> 3;
    size_t bytecount = byteend - bytepos;

    uint64_t code_shifted = magnitude << (bitpos & 7);
    for (size_t j = 0; j < bytecount; ++j)
    {
        assert(bytepos + j < bitstream_len);
        bitstream[bytepos + j] |= code_shifted >> (j * 8);
    }

    if (magnitude != 0)
    {
        if (value < 0)
        {
            assert((sign_bitpos >> 3) < bitstream_len);
            bitstream[sign_bitpos >> 3] |= 1 << (sign_bitpos & 7);
        }
        ++sign_bitpos;
    }
}

static bool test_all_signs(size_t initial_bitpos)
{
    // always use bitlen=1 and 8*256 values for this test
    static const int ngroups = 256; // all possible patterns of 8 lanes having/not having data
    static const int count = ngroups * 8;    

    int num_bands = 1;
    U32 bands[2] = {0, count + 3}; // set up the bands so we always use the first threshold (1.0)

    size_t startup_bitcount = 7 + FXPBITS*2; // 1 band and 2 initial samples
    size_t val_bitcount = count; // 1 bit per val for this test.
    size_t sign_bitcount = count; // sign bits are max 1 per
    size_t rle_bitcount = ngroups * 5;

    size_t write_pos = initial_bitpos;
    size_t end_pos_conservative = startup_bitcount + val_bitcount + rle_bitcount + sign_bitcount + 128; // +128 is extra padding for our sloppy reads

    size_t bitstream_len = (end_pos_conservative + 7) >> 3;
    U8* bitstream = (U8*)malloc(bitstream_len);
    memset(bitstream, 0, bitstream_len);
    
    // encode the first two samples
    encode_bits(bitstream, bitstream_len, write_pos, 0, FXPBITS);
    write_pos += FXPBITS;
    encode_bits(bitstream, bitstream_len, write_pos, 0, FXPBITS);
    write_pos += FXPBITS;

    // Encode the threshold index
    encode_bits(bitstream, bitstream_len, write_pos, 0, 7);
    write_pos += 7;

    int sign = 1;
    size_t sign_bitpos = write_pos;
    for (int i = 0; i < count; ++i)
    {
        int grp = i / 8; // index of group
        int pos_in_grp = i % 8;

        if (pos_in_grp == 0)
        {
            // 8 is the VQLENGTH, so we can encode single length runs

            // we start values where the last signs end.
            write_pos = sign_bitpos;

            // 0 means its a single length run
            encode_bits(bitstream, bitstream_len, write_pos, 0, 1);
            write_pos++;
            // bitlen = 1
            encode_bits(bitstream, bitstream_len, write_pos, 1, 4);
            write_pos +=4;

            // each run has the signs at the end
            sign_bitpos = write_pos + 1 * 8;
        }

        int value = 0;
        if (grp & (1 << pos_in_grp))
        {
            // just keep toggling signs, good enough pattern for this test (I hope)
            value = sign;
            sign = -sign;
        }

        encode_value(bitstream, bitstream_len, write_pos, sign_bitpos, value, 1);
        write_pos++;
        assert(write_pos <= sign_bitpos);
    }

    float output_floats[count + 2 + 8]; // entries + the first two + memset overwrite
    float ref_floats[count + 2 + 8];

    BINKVARBITS vb;
    BinkVarBitsOpen(vb, bitstream);
    BinkVarBitsUse(vb, (U32)initial_bitpos);
    read_channel_data_2(output_floats, count + 2, num_bands, &vb, bands, bitstream + bitstream_len);

    read_channel_data_ref(ref_floats, bitstream, bitstream_len, count + 2, initial_bitpos);
    
    bool failed = false;
    for (int i=0; i < count + 2; i++)
    {
        if (output_floats[i] != ref_floats[i])
        {
            printf("test_all_signs failed initial_pos: %d index %d\n", (int)initial_bitpos, i);
            failed = true;
            break;
        }
    }
    free(bitstream);

    return failed;
}

static bool test_all_codes(size_t initial_bitpos, int bitlen)
{
    assert(bitlen >= 1 && bitlen <= 15);
    int numcodes = 1 << bitlen;
    if (numcodes < 8)
        numcodes = 8;

    int num_bands = 1;
    U32 bands[2] = { 0, (U32)numcodes + 3 }; // set up the bands so we always use the first threshold (1.0)

    size_t startup_bitcount = 7 + FXPBITS * 2; // 1 band and 2 initial samples
    size_t val_bitcount = numcodes * bitlen; // 1 bit per val for this test.
    size_t sign_bitcount = numcodes; // sign bits are max 1 per
    size_t rle_bitcount = (numcodes >> 3) * 5;

    size_t write_pos = initial_bitpos;
    size_t end_pos_conservative = initial_bitpos + startup_bitcount + val_bitcount + rle_bitcount + sign_bitcount + 128; // +128 is extra padding for our sloppy reads

    size_t bitstream_len = (end_pos_conservative + 7) >> 3;
    U8* bitstream = (U8*)malloc(bitstream_len);
    memset(bitstream, 0, bitstream_len);

    // encode the first two samples
    encode_bits(bitstream, bitstream_len, write_pos, 0, FXPBITS);
    write_pos += FXPBITS;
    encode_bits(bitstream, bitstream_len, write_pos, 0, FXPBITS);
    write_pos += FXPBITS;

    // Encode the threshold index
    encode_bits(bitstream, bitstream_len, write_pos, 0, 7);
    write_pos += 7;

    size_t sign_bitpos = write_pos;
    for (int i = 0; i < numcodes; ++i)
    {
        int pos_in_grp = i % 8;

        if (pos_in_grp == 0)
        {
            // 8 is the VQLENGTH, so we can encode single length runs

            // we start values where the last signs end.
            write_pos = sign_bitpos;

            // 0 means its a single length run
            encode_bits(bitstream, bitstream_len, write_pos, 0, 1);
            write_pos++;

            encode_bits(bitstream, bitstream_len, write_pos, bitlen, 4);
            write_pos += 4;

            // each run has the signs at the end
            sign_bitpos = write_pos + bitlen * 8;
        }

        int code = i & ((1 << bitlen) - 1);

        encode_value(bitstream, bitstream_len, write_pos, sign_bitpos, code, bitlen);
        write_pos += bitlen;
        assert(write_pos <= sign_bitpos);
    }

    float* output_floats = (float*)malloc(sizeof(float) * (numcodes + 2) + 32); // entries + the first two. + memset overwrite
    float* ref_floats = (float*)malloc(sizeof(float) * (numcodes + 2) + 32); // entries + the first two.
    
    U64 start_ticks = baue_ticks();

    for (int i = 0; i < 200; i++)
    {
        BINKVARBITS vb;
        BinkVarBitsOpen(vb, bitstream);
        BinkVarBitsUse(vb, (U32)initial_bitpos);
        read_channel_data_2(output_floats, numcodes + 2, num_bands, &vb, bands, bitstream + bitstream_len);
    }

    U64 end_ticks = baue_ticks();
    printf("run time x200 (%d / %d) : %llu\n", (int)initial_bitpos, bitlen, end_ticks - start_ticks);


    read_channel_data_ref(ref_floats, bitstream, bitstream_len, numcodes + 2, initial_bitpos);

    bool failed = false;
    for (int i = 0; i < numcodes + 2; i++)
    {
        if (output_floats[i] != ref_floats[i])
        {
            printf("test_all_codes failed initial_pos: %d index %d\n", (int)initial_bitpos, i);
            failed = true;
            break;
        }
    }

    free(bitstream);
    free(output_floats);
    free(ref_floats);

    return failed;
}

// asserts on failure.
#ifdef __RADINDLL__
RADEXPFUNC bool RADEXPLINK BinkAudioDecompressTestReadChanData()
#else
RADDEFFUNC bool RADLINK BinkAudioDecompressTestReadChanData()
#endif
{
    bool failed = false;
    CPU_check(0, 0);
    for (int bitpos = 0; bitpos < 8; ++bitpos)
    {
        failed |= test_all_signs(bitpos);
    }

    for (int bitlen = 1; bitlen <= 15; ++bitlen)
    {
        for (int bitpos = 0; bitpos < 8; ++bitpos)
        {
            failed |= test_all_codes(bitpos, bitlen);
        }
    }

    return failed;
}

#endif // TEST_READ_CHAN_DATA