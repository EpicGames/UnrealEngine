// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef __BINKACDH__
#define __BINKACDH__

#ifndef __RADRR_COREH__
  #include "rrCore.h"
#endif

//===========================================
//  decoding API
//===========================================

#ifdef WRAP_PUBLICS
#define bamerge(name) RR_STRING_JOIN(name,WRAP_PUBLICS)
#define BinkAudioDecompressMemory          bamerge(BinkAudioDecompressMemory)
#define BinkAudioDecompressOpen            bamerge(BinkAudioDecompressOpen)
#define BinkAudioDecompress                bamerge(BinkAudioDecompress)
#define BinkAudioDecompressResetStartFrame bamerge(BinkAudioDecompressResetStartFrame)
#define BinkAudioDecompressOutputSize      bamerge(BinkAudioDecompressOutputSize)
#endif

typedef struct BINKAUDIODECOMP * HBINKAUDIODECOMP;

#define BINKACNODEINTERLACE 2
#define BINKAC20 4

#define BINKACD_EXTRA_INPUT_SPACE 72 // extra padding after inpend, that, given random data, Bink might read past (very unlikely, but possible)

// with _normal_ data, we can read past the InputBuffer by 16 bytes
// due to vector bit decoding. It's highly unlikely that it reads that far,
// however _some_ amount of reading is all but certain.
#define BINK_UE_DECODER_END_INPUT_SPACE 16

// how much memory should be allocated for the BinkAudioDecompressOpen mem ptr?
RADDEFFUNC U32 RADLINK BinkAudioDecompressMemory( U32 rate, U32 chans, U32 flags );

// open and initialize a decompression stream
RADDEFFUNC U32 RADLINK BinkAudioDecompressOpen( void * mem, U32 rate, U32 chans, U32 flags );

typedef struct BINKAC_OUT_RINGBUF
{
  void * outptr;         // pointer within outstart and outend to write to (after return, contains the new end output ptr)
  void * outstart;
  void * outend;
  
  U32   outlen;          // unused available (starting at outptr, possibly wrapping) - (after return, how many bytes copied)
                         // outlen should always be a multiple of 16

  U32   eatfirst;        // remove this many bytes from the start of the decompressed data (after return, how many bytes left)
                         // eatfirst should always be a multple of 16                       

  U32   decoded_bytes;   // from the input stream, how many bytes decoded (usually outlen, unless outlen was smaller than a frame - that is, when not enough room in the ringbuffer)
} BINKAC_OUT_RINGBUF;

typedef struct BINKAC_IN
{
  void const * inptr;      // pointer to input data (after return, contains new end input ptr)
  void const * inend;      // end of input buffer (there should be at least BINKACD_EXTRA_INPUT_SPACE bytes after this ptr)
} BINKAC_IN;

//do the decompression - supports linear or ringbuffers (outptr==outstart on non-ring), will clamp, if no room
RADDEFFUNC void RADLINK BinkAudioDecompress(void* mem, BINKAC_OUT_RINGBUF * output, BINKAC_IN * compress_input);

// resets the start flag to prevent blending in the last decoded frame.
RADDEFFUNC void RADLINK BinkAudioDecompressResetStartFrame(void* mem);

// how much memory will decompress touch of the output (not how much was output - that's decoded_bytes)
RADDEFFUNC U32 RADLINK BinkAudioDecompressOutputSize(void* mem);

// what is the maximum data that can be touched of the output for ANY Bink file
#define BinkAudioDecompressOutputMaxSize() ( 2048 * sizeof(S16) * 2 )  // 2 is channels

#endif
