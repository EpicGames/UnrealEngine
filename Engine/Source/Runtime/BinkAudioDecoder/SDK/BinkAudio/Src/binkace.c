// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef __RADRR_COREH__
  #include "rrCore.h"
#endif

#include "binkace.h"

#include <string.h>
#define rrmemsetzero(d,c) memset(d,0,c) // use for small zero clears
#define rrmemmovebig memmove // use for large copies (>512 bytes) - can overlay

#include "radmath.h"
#include "binkace.h"
#include "varbits.h"
#include "popmal.h"
#include "radfft.h"

#define AUDIOFLOAT              F32
#define AUDIOSAMPLE             F32
#define AUDIOTABLESAMPLE        F32

//#define DEBUGSTACKVARS

#ifdef BIG_OLE_FFT // never set for ue binka
#define MAXBUFFERSIZE     4096
#else
#define MAXBUFFERSIZE     2048
#endif
#define MAXBUFFERSIZEHALF   ( MAXBUFFERSIZE / 2 )
#define MAXCHANNELS         2
#define WINDOWRATIO         16

#define TOTBANDS 25

#define FXPBITS 29

#define VQLENGTH 8

#define RLEBITS 4
#define MAXRLE (1<<RLEBITS)

static U8 rlelens[ MAXRLE ] =
{
  2,3,4,5, 6,8,9,10, 11,12,13,14, 15,16,32,64
};

static U32 bandtopfreq[ TOTBANDS ]=
{
  0, 100, 200, 300, 400, 510, 630, 770, 920, 1080, 1270, 1480, 1720, 2000,
  2320, 2700, 3150, 3700, 4400, 5300, 6400, 7700, 9500, 12000, 15500
};

static AUDIOFLOAT RADINLINE Undecibel( AUDIOFLOAT d )
{
  return( ( AUDIOFLOAT ) radpow( 10, d * 0.10f ) );
}

#include "undeci.inc"

// ----

typedef struct BINKAUDIOCOMP
{
  U32 transform_size;
  U32 buffer_size;
  U32 window_size;
  S32 chans;
  U32 flags;
  F32 transform_size_root;
  AUDIOFLOAT threshold_curve_adj;
  S16* inp;
  S16* inpr;
  S16* outp;
  U32 start_frame;
  U32 num_bands;
  U32 given;
  U32 got;
  U32 * bands;
  AUDIOFLOAT * renorm;
  AUDIOFLOAT * last_pow_pha;
  AUDIOFLOAT * ath;
  AUDIOFLOAT * band_ath;
  void* (*memalloc)(UINTa bytes);
  void (*memfree)(void* ptr);
} BINKAUDIOCOMP;


// center frequency of each band (here just for reference)
//static U32 frequency[TOTBANDS]={50,150,250,350,450,570,700,840,1000,1170,1370,1600,1850,
//                                2150,2500,2900,3400,4000,4800,5800,7000,8500,10500,13500,18775};

static AUDIOFLOAT spreadfactors[TOTBANDS*2]=
{
   0.0000000F, 0.0000000F, 0.0000000F, 0.0000000F, 0.0000000F,
   0.0000000F, 0.0000000F, 0.0000000F, 0.0000000F, 0.0000000F,
   0.0000000F, 0.0000000F, 0.0000000F, 0.0000000F, 0.0000000F,
   0.0000000F, 0.0000000F, 0.0000000F, 0.0000000F, 0.0000000F,
   0.0000000F, 0.0000000F, 0.0000038F, 0.0010497F, 0.1347701F,

   1.0000000F, 0.0787500F, 0.0146941F, 0.0052495F, 0.0005273F,
   0.0000507F, 0.0000048F, 0.0000004F, 0.0000000F, 0.0000000F,
   0.0000000F, 0.0000000F, 0.0000000F, 0.0000000F, 0.0000000F,
   0.0000000F, 0.0000000F, 0.0000000F, 0.0000000F, 0.0000000F,
   0.0000000F, 0.0000000F, 0.0000000F, 0.0000000F, 0.0000000F
};


static AUDIOFLOAT RADINLINE Decibel( AUDIOFLOAT d )
{
  return( (AUDIOFLOAT) ( 10.0 * radlog10( d ) ) );
}

//==============================================================================
//   encoding functions
//==============================================================================

static U32 calcbitlevels( U16 * levels, S16 * coeffs, U32 buffersize )
{
  U32 i;
  U16* l = levels;

  for( i = 2 ; i < buffersize ; )
  {
    U32 j,lev;

    j = i + VQLENGTH;
    if ( j > buffersize)
      j = buffersize;

    lev = 0;
    for( ; i < j ; i++ )
    {
      U32 cur = getbitlevelvar( radabs( coeffs[ i ] ) );
      if ( cur > lev )
        lev = cur;
    }
    *l++ = (U16) lev;
  }

  return (U32)( l - levels );
}


static void rlebitlevels( U16 * levels, U32 levlen )
{
  U32 lev, len, maxlen, test;
  S32 rle;
  U16 * levs, * olevs;

  maxlen = levlen;
  levs = levels;
  olevs = levels;
  test = 0;
  do
  {
    rle = -1;
    len = 1;
    lev = levs[ 0 ];

    while ( ( len < maxlen ) && ( rle < ( MAXRLE - 1 ) ) )
    {
      U32 nextlen, j;

      nextlen = rlelens[ rle + 1 ];
      if ( nextlen > maxlen )
        nextlen = maxlen;

      for( j = len ; j < nextlen ; j++ )
        if ( levs[ j ] != lev )
          goto endrle;

      len = nextlen;
      ++rle;
    }

endrle:

    *olevs++ = (U16) ( ( rle << 8 ) + lev );
    maxlen -= len;
    levs += len;
  } while ( maxlen );
}


static void encodebitlevels( VARBITS * vb,
                             U16 * levels,
                             S16 * coeffs,
                             U32 buffersize )
{
  // bink audio 1 encoder -- sign bits directly follow the coeff bits
  U32 i;
  U16* l = levels;

  i = 2;

  do
  {
    U32 rle, len, lev;

    lev = *l++;
    rle = lev >> 8;
    lev &= 255;

    if ( rle == 255 )
    {
      VarBitsPuta0( *vb );
      len = VQLENGTH;
    }
    else 
    {
      VarBitsPuta1( *vb );
      VarBitsPut( *vb, rle, RLEBITS );
      len = ( (U32) rlelens[ rle ] ) * VQLENGTH;
    }
    VarBitsPut( *vb, lev, 4);

    if ( len > ( buffersize - i ) )
      len = buffersize - i;

    if ( lev == 0 )
      i += len;
    else
    {
      while ( len-- )
      {
        VarBitsPut( *vb, radabs( coeffs[ i ] ), lev );

        if ( coeffs[ i ] )
          VarBitsPut1( *vb, ( coeffs[ i ] < 0 ) );

        ++i;
      }
    }

  } while ( i < buffersize );
}

static void encodebitlevels2( VARBITS * vb,
                             U16 * levels,
                             S16 * coeffs,
                             U32 buffersize )
{
    // bink audio 2 encoder -- sign bits are grouped after all coeffs in a run.
	U32 i;
	U16* l = levels;

	i = 2;

	do
	{
		U32 rle, len, lev;

		lev = *l++;
		rle = lev >> 8;
		lev &= 255;

		if (rle == 255)
		{
			VarBitsPuta0(*vb);
			len = VQLENGTH;
		}
		else
		{
			VarBitsPuta1(*vb);
			VarBitsPut(*vb, rle, RLEBITS);
			len = ((U32)rlelens[rle]) * VQLENGTH;
		}
		VarBitsPut(*vb, lev, 4);

		if (len > (buffersize - i))
			len = buffersize - i;

        if (lev == 0)
        {
            i += len;
        }
		else
		{
            U32 start = i;
            U32 startlen = len;
            while (len--)
            {
                VarBitsPut(*vb, radabs(coeffs[i]), lev);
                ++i;
            }

            len = startlen;
            i = start;
			while (len--)
			{
				if (coeffs[i])
					VarBitsPut1(*vb, (coeffs[i] < 0));
				++i;
			}
		}

	} while (i < buffersize);
}


static U32 ftofxp( F32 val )
{
  U32 b, v;
  F32 f;
  f = (F32) radfabs( val );
  v = (U32) radfloor(f);
  b = getbitlevelvar( v );

  b |= ( (U32) ( (F64) f * (F64) ( 1 << ( 23 - b ) ) ) ) << 5;

  if ( val < 0 )
    b |= 0x10000000;

  return( b );
}


static AUDIOFLOAT calctonality( U32 transform_size,
                                U32 startband,
                                U32 endband,
                                AUDIOFLOAT* power,
                                AUDIOFLOAT* phase,
                                AUDIOFLOAT* last1pow )
{
  // simple model described in Applications of Digital Signal Processing
  AUDIOFLOAT t, n;
  U32 i;

  AUDIOFLOAT * last1pha = last1pow + ( transform_size / 2 );
  AUDIOFLOAT * last2pow = last1pha + ( transform_size / 2 );
  AUDIOFLOAT * last2pha = last2pow + ( transform_size / 2 );

  t=0;
  n=0;

  for( i = startband ; i < endband ; i++ )
  {
    AUDIOFLOAT m, mp, pp, c, pow, a, b;

    m = (AUDIOFLOAT) radfsqrt( power[ i ] );

    mp = last1pow[ i ] + last1pow[ i ] - last2pow[ i ];
    pp = last1pha[ i ] + last1pha[ i ] - last2pha[ i ];

    last2pow[ i ] = last1pow[ i ];
    last2pha[ i ] = last1pha[ i ];
    last1pow[ i ] = m;
    last1pha[ i ] = phase[ i ];

    a = mp - m;
    b = pp - phase[ i ];

    pow = (AUDIOFLOAT) ( m + radfabs( mp ) );
    if ( pow < 2.5F )
    {
      c = 0.5f;
    }
    else
    {
      c = (AUDIOFLOAT) ( radfsqrt( a*a + b*b ) / pow );
    }

    if ( c > 0.5F )
      c = 0.5F;
    else if ( c < 0.05F)
      c = 0.05F;

    c = (AUDIOFLOAT)( -0.43F * ranged_log_0p05_to_0p5( c ) ) - 0.29F;

    t += c * m;
    n += m;
  }

  if ( n < ( ( (AUDIOFLOAT) ( endband - startband ) ) * 2.5f ) )
    t= 0.0f;
  else
    t = t / n;

  return( t );
}


static void multiply_samples_by_scalar( AUDIOFLOAT * samples,
                                        U32 number,
                                        F32 scalar )
{
  U32 i;
  AUDIOFLOAT * f;

  f = samples;
  for( i = number; i; i-- )
  {
    (*f) *= ( (AUDIOFLOAT) scalar );
    ++f;
  }
}


static void calc_power_phase( AUDIOFLOAT * samples,
                              AUDIOFLOAT * power,
                              AUDIOFLOAT * phase,
                              U32 num )
{
  U32 i;
  AUDIOFLOAT * f;

  f = samples;
  for( i = 0; i < num; i++ )
  {
    power[ i ] = f[ 0 ] * f[ 0 ] + f[ 1 ] * f[ 1 ];
    if ( power[ i ] <= 0.0005F )
    {
      phase[i] = 0.0F;
      power[i] = 0.0005F;
    }
    else
    {
      // imaginary is sign inverted in our fft
      phase[ i ] = (AUDIOFLOAT) radatan2( -f[ 1 ], f[ 0 ] );
    }
    f += 2;
  }
}


static void calc_band_power( AUDIOFLOAT * band_power,
                             AUDIOFLOAT * power,
                             U32 * bands,
                             U32 num_bands )
{
  U32 i;

  for ( i = 0 ; i < num_bands ; i++ )
  {
    U32 j;

    band_power[ i ] = 0.0;

    for( j = bands[ i ] ; j < bands[ i + 1 ] ; j++ )
    {
      band_power[ i ] += power[ j ];
    }
  }
}


static void calc_band_tonality( AUDIOFLOAT * band_tonality,
                                AUDIOFLOAT * power,
                                AUDIOFLOAT * phase,
                                AUDIOFLOAT * last_pow_pha,
                                U32 * bands,
                                U32 num_bands,
                                U32 transform_size )
{
  U32 i;

  for ( i = 0 ; i < num_bands ; i++ )
  {
    U32 j;
    AUDIOFLOAT tone_vs_noise;

    tone_vs_noise = calctonality( transform_size, bands[i], bands[i+1], power, phase, last_pow_pha );

    band_tonality[ i ] = 0.0;

    for( j = bands[ i ] ; j < bands[ i + 1 ] ; j++ )
    {
      band_tonality[ i ] += power[ j ] * tone_vs_noise;
    }
  }
}


static void simulate_spreading( AUDIOFLOAT * out,
                                AUDIOFLOAT * in,
                                U32 num_bands )
{
  U32 i;

  for ( i = 0 ; i < num_bands ; i++)
  {
    out[ i ] = 0.0;
  }

  for ( i = 0 ; i < num_bands ; i++)
  {
    U32 j;

    // spread across the bands
    for ( j = 0 ; j < num_bands; j++ )
    {
      out[ j ] += ( in[ i ] * spreadfactors[ (S32) j - (S32) i + TOTBANDS ] );
    }
  }
}

#define CAREFULBANDS 8

static void calc_thresholds( AUDIOFLOAT * threshold,
                             AUDIOFLOAT lossy,
                             AUDIOFLOAT threshold_curve_adj,
                             AUDIOFLOAT * spread_tonality,
                             AUDIOFLOAT * spread_power,
                             U32 * bands,
                             U32 num_bands )
{
  U32 i;

  for ( i = 0 ; i < num_bands ; i++ )
  {
    AUDIOFLOAT range, db_level;

    // normalize the tonality (non-normalized after spreading)
    if ( spread_power[ i ] <= 0.0005f )
      range = 0.0f;
    else
      range = ( spread_tonality[ i ] / spread_power[ i ] );

    // clamp the tonality to the range
    if ( range < 0.0F )
      range = 0.0F;
    else if ( range > 1.0F)
      range = 1.0F;

    // calculate the masking difference (which is deltaed from the signal strength)
    db_level = ( range * ( 15.0F + (AUDIOFLOAT) i ) ) +
               ( ( 1.0F - range ) * 6.0F );

    // adjust to move directly off the edge of the curve
    db_level += threshold_curve_adj;

    if ( i < CAREFULBANDS )
      db_level += threshold_curve_adj;

    // factor in our lossy level
    db_level -= lossy * 2;
    if ( db_level < 0.0f ) db_level = 0.0f;
    
//    printf( "%i %12.3f %12.3f\n", i, range, db_level );

    threshold[i]=db_level;
  }
}


static void calc_best_quant( U32 * best_qlevel,
                             AUDIOFLOAT * samples,
                             AUDIOFLOAT * threshold,
                             U32 * bands,
                             U32 num_bands )
{
  U32 i;

  for( i = 0 ; i < num_bands ; i++ )
  {
    U32 ll, hl, nl;  
    AUDIOFLOAT best_diff, max_value, band_power, band_thres;
    U32 j;
    AUDIOFLOAT * f;

    best_qlevel[ i ] = 0;

    ll = 0;
    hl = 96;
    nl = 48;
    max_value = 0.0f;
    band_power = 0.0f;
    best_diff = 0.0f;

    f = samples + ( bands[ i ] * 2 );
    for( j = bands[ i ] ; j < bands[ i + 1 ] ; j++ )
    {
      AUDIOFLOAT fa,fb;

      fa = f[ 0 ];
      fb = f[ 1 ];

      band_power += ( ( fa * fa ) + ( fb * fb ) );

      fa = (AUDIOFLOAT) radfabs( fa );
      fb = (AUDIOFLOAT) radfabs( fb );

      if ( max_value < fa ) max_value = fa;
      if ( max_value < fb ) max_value = fb;

      f += 2;
    }

    // find the top end quantization
    for( j = 94; j > 0 ; j-- )
      if ( max_value >= bink_Undecibel_table[ j ] )
        break;
    hl = j + 2;

    nl = ( hl + ll ) / 2;

    // subtract db from the band_power of the 
    band_thres = band_power / Undecibel( threshold[ i ] );
    
    band_thres = band_thres + ( band_thres * 0.0000001f );

    // we're going to binary search for the best quant level
    for(;;)
    {
      AUDIOFLOAT testthr, testthrdiv, tot_diff;
      U32 l;

      l = nl;

      testthr = bink_Undecibel_table[ l ];
      testthrdiv  = (AUDIOFLOAT) ( 1.0 / testthr );

      // add up the error for this threshold
      f = samples + ( bands[ i ] * 2 );
      tot_diff = (AUDIOFLOAT) 0;

      for( j = bands[ i ] ; j < bands[ i + 1 ] ; j++ )
      {
        AUDIOFLOAT temp, samp;

        
        // quantize and then unquantize
        samp = (AUDIOFLOAT) radfabs( f[ 0 ] );
        temp = (AUDIOFLOAT) radfloor( samp * testthrdiv + 0.5f );
        if ( temp <= -32767.0 )
          temp = -32767.0F;
        else if ( temp >= 32767.0 )
          temp = 32767.0F;
        temp *= testthr;

        // take the difference between orig and quant and square the diff
        temp -= samp;
        tot_diff += (AUDIOFLOAT)( temp * temp );


        // quantize and then unquantize
        samp = (AUDIOFLOAT) radfabs( f[ 1 ] );
        temp = (AUDIOFLOAT) radfloor( samp * testthrdiv  + 0.5f );
        if ( temp <= -32767.0 )
          temp = -32767.0F;
        else if ( temp >= 32767.0 )
          temp = 32767.0F;
        temp *= testthr;

        // take the difference between orig and quant and square the diff
        temp -= samp;
        tot_diff += (AUDIOFLOAT)( temp * temp );
        f += 2;
      }

      // the difference less than our threshold?
      if ( tot_diff <= band_thres )
      {
        // is the difference larger than out best difference (but still less than our threshold?)
        if ( tot_diff >= best_diff )
        {
          best_diff = tot_diff;
          best_qlevel[ i ] = l;
        }

        ll = l;
      }
      else
      {
        // move the binary search down
        hl = l;
      }

      nl = ( ( hl + ll ) / 2 );

      if ( nl == l )
        break;

    }

//{ static U32 highest = 0; if (best_qlevel[i]> highest) highest = best_qlevel[i];  printf("%3i %12.3f %12.3f best: %3i (%3i)\n", i, threshold[i], Undecibel_table[ best_qlevel[ i ] ], best_qlevel[ i ], highest ); }
  }
}


static void quantize_fft_samples( S16 * out,
                                  AUDIOFLOAT * samples,
                                  AUDIOFLOAT * threshold_div,
                                  U32 * bands,
                                  U32 num_bands,
                                  AUDIOFLOAT * ath,
                                  AUDIOFLOAT lossy )
{
  U32 i;
  AUDIOFLOAT * f;
  S16 * op;

  f = samples;
  op = out;

  for ( i = 0; i < num_bands ; i++ )
  {
    U32 j;

    for ( j = ( bands[ i ] * 2 ) ; j < ( bands[ i + 1 ] * 2 ) ; j++ )
    {
      AUDIOFLOAT temp;

      temp = ath[ j ];

      if ( i >= CAREFULBANDS )
        temp += lossy * 10.0f;

      temp = (AUDIOFLOAT) Undecibel( temp );

      if ( ( f[ 0 ] * f[ 0 ] ) < temp )
        *op++ = 0;
      else
      {
        AUDIOFLOAT samp;

        samp = f[ 0 ];
        temp = (AUDIOFLOAT) radfloor( radfabs( samp ) * threshold_div[ i ] + 0.5f );
        if ( temp > 32767 ) temp = 32767;
        if ( samp < 0.0f ) temp = -temp;
        
        *op++ = (S16)temp;
      }

      ++f;
    }
  }
}


static void do_perceptual( AUDIOFLOAT * threshold,
                           AUDIOFLOAT lossy,
                           AUDIOFLOAT threshold_curve_adj,
                           AUDIOFLOAT * samples,
                           U32 transform_size,
                           AUDIOFLOAT * last_pow_pha,
                           U32 * bands,
                           U32 num_bands )
{
  #ifdef DEBUGSTACKVARS

  #ifdef __RADFINAL__
  #error "You have debug stack turned on!"
  #endif
  AUDIOFLOAT * power;
  AUDIOFLOAT * phase;
  AUDIOFLOAT * band_power;
  AUDIOFLOAT * band_tonality;
  AUDIOFLOAT * spread_power;
  AUDIOFLOAT * spread_tonality;
  char pmbuf[ PushMallocBytesForXPtrs( 8 ) ];
  
  pushmallocinit( pmbuf, 8 );
  pushmalloc( pmbuf, &power, 4 * ( transform_size / 2 ) );
  pushmalloc( pmbuf, &phase, 4 * ( transform_size / 2 ) );
  pushmalloc( pmbuf, &band_power, 4 * num_bands );
  pushmalloc( pmbuf, &band_tonality, 4 * num_bands );
  pushmalloc( pmbuf, &spread_power, 4 * num_bands );
  spread_tonality = popmalloc( pmbuf, 4 * num_bands );

  #else

  AUDIOFLOAT power[ MAXBUFFERSIZE ];
  AUDIOFLOAT phase[ MAXBUFFERSIZE ];
  AUDIOFLOAT band_power[ TOTBANDS ];
  AUDIOFLOAT band_tonality[ TOTBANDS ];
  AUDIOFLOAT spread_power[ TOTBANDS ];
  AUDIOFLOAT spread_tonality[ TOTBANDS ];

  #endif

  rrassert( MAXBUFFERSIZE >= ( transform_size / 2 ) );
  rrassert( TOTBANDS >= num_bands );


  // since we separately send these values, clear them out so they don't effect anything
  samples[ 0 ] = 1.0f;
  samples[ 1 ] = 0.0f;
  power[ 0 ] = 1.0F;
  phase[ 0 ] = 0.0F;

  // calculate the power and phase for each sample
  calc_power_phase( samples + 2, power + 1, phase + 1, ( transform_size / 2 ) - 1 ); // minus 1, since we do the first one by hand

  // calculate the total power for each band
  calc_band_power( band_power, power, bands, num_bands );

  // calculate the tonality each band
  calc_band_tonality( band_tonality, power, phase, last_pow_pha, bands, num_bands, transform_size );

  // Simulate the spreading activation of sound in the ear
  simulate_spreading( spread_power, band_power, num_bands );
  simulate_spreading( spread_tonality, band_tonality, num_bands );

  // Generate masking threshold from spread spectral information
  calc_thresholds( threshold, lossy, threshold_curve_adj, spread_tonality, spread_power, bands, num_bands );

  #ifdef DEBUGSTACKVARS
    popfree( spread_tonality );
  #endif
}


static void clamp_best_quant_to_ath( U32 * best_qlevel, AUDIOFLOAT * band_ath, U32 num_bands, AUDIOFLOAT lossy )
{
  U32 i;

  for( i = 0 ; i < num_bands ; i++ )
  {
    AUDIOFLOAT band_db, ath;

    ath = band_ath[ i ] + lossy;

    band_db = ( (AUDIOFLOAT) (U8) best_qlevel[ i ] ) * 0.664F;

    if ( ath > band_db )
    {
      best_qlevel[ i ] = (U32) ( ( ath + 0.3319F ) / 0.664F );
    }
    
    if (best_qlevel[i] > 95)
        best_qlevel[i] = 95;
  }
}


static void encode_one_channel( U32 transform_size,
                                VARBITS * vb,
                                AUDIOFLOAT * samples,
                                U32 * best_qlevel,
                                U32 num_bands,
                                U32 * bands,
                                AUDIOFLOAT * ath,
                                F32 lossy,
                                U32 is_ba2)
{
  U32 i;
  #ifdef DEBUGSTACKVARS
  S16 * coeffs;
  U16 * levels;
  AUDIOFLOAT * threshold_mult;
  char pmbuf[ PushMallocBytesForXPtrs( 8 ) ];
  
  pushmallocinit( pmbuf, 8 );
  pushmalloc( pmbuf, &threshold_mult, 4 * num_bands );
  pushmalloc( pmbuf, &coeffs, 2 * transform_size );
  levels = popmalloc( pmbuf, 2 * ( ( transform_size + VQLENGTH ) / VQLENGTH ) );

  #else

  AUDIOFLOAT threshold_mult[ TOTBANDS ];
  S16 coeffs[ MAXBUFFERSIZE ];
  U16 levels[ ( ( MAXBUFFERSIZE + VQLENGTH ) / VQLENGTH ) ];

  #endif

  rrassert( TOTBANDS >= num_bands );
  rrassert( ( MAXBUFFERSIZE ) >= transform_size );

  // unquant the threshold for compression
  for ( i = 0 ; i < num_bands ; i++ )
  {
    threshold_mult[ i ] = ( 1.0F / bink_Undecibel_table[ best_qlevel[ i ] ] );
  }

  // quantize the samples using the levels we found
  quantize_fft_samples( coeffs, samples, threshold_mult, bands, num_bands, ath, lossy );

  //calculate all of the VQ lengths
  i = calcbitlevels( levels, coeffs, transform_size );

  //rle the bit levels
  rlebitlevels( levels, i );

  //now encode the bits
  if (is_ba2)
    encodebitlevels2(vb, levels, coeffs, transform_size);
  else
    encodebitlevels( vb, levels, coeffs, transform_size );

  #ifdef DEBUGSTACKVARS
  popfree( levels );
  #endif
}


static void load_samples( AUDIOFLOAT* samples,
                          S16* in_data,
                          U32 in_bytes,
                          U32 chan,
                          U32 chans,
                          U32 transform_size )
{
  AUDIOFLOAT * f;
  U32 num_samps;
  S16* id;
  AUDIOFLOAT temp;
  U32 i;

  num_samps = ( in_bytes / 2 ) / chans;

  f = samples;
  id = in_data + chan;
  for ( i = num_samps; i ; i-- )
  {
    *f++ = *id;
    id += chans;
  }

  // pad the out the buffer (duplicate the final sample)
  temp = f[ -1 ];
  for ( i = num_samps ; i < transform_size ; i++ )
  {
    *f++ = temp;
  }
}


static F32 simple_ease( F32 in ) // takes linear 0 to 1, returns smooth curve
{
  F32 sqr = in * in;

  // 3x^2 - 2x^3
  return( ( 3.0f * sqr ) - ( 2.0f * sqr * in ) );
}


static void ramp_samples( AUDIOFLOAT * samples, F32 start_weight, F32 end_weight, U32 num )
{
  U32 i;

  for( i = 0 ; i < num ; i++ )
  {
    samples[ i ] = samples[ i ] *
                   simple_ease( ( ( start_weight * (AUDIOFLOAT) ( num - i ) ) +
                                  ( end_weight * (AUDIOFLOAT) i )
                                ) / (AUDIOFLOAT) num );

    if ( ( samples[ i ] >= -0.001 ) || ( samples[ i ] <= 0.001 ) )
      samples[ i ] = 0;
  }
}


//encode the data into an output buffer and return the length (in bytes)
static U32 Percept( U32 transform_size,
                    F32 transform_size_root,
                    U32 chans,
                    U32 flags,
                    AUDIOFLOAT * last_pow_pha,
                    U32 lossy_level,
                    F32 threshold_curve_adj,
                    void * buf,
                    S16 * in_data,
                    U32 in_bytes,
                    U32 num_bands,
                    U32 * bands,
                    AUDIOFLOAT * band_ath,
                    AUDIOFLOAT * ath,
                    AUDIOFLOAT * renorm )
{
  VARBITS vb;
  U32 i;
  AUDIOFLOAT lossy;

  #ifdef DEBUGSTACKVARS

  U32 * best_qlevel;
  AUDIOFLOAT * samples;
  AUDIOFLOAT * threshold;
  char pmbuf[ PushMallocBytesForXPtrs( 8 ) ];
  
  pushmallocinit( pmbuf, 8 );
  pushmalloc( pmbuf, &best_qlevel, 4 * TOTBANDS );
  pushmalloc( pmbuf, &threshold, 4 * TOTBANDS );
  samples = popmalloc( pmbuf, 4 * transform_size * chans );

  #else

  U32 best_qlevel[ TOTBANDS ];
  AUDIOFLOAT threshold[ TOTBANDS ];
  RAD_ALIGN( AUDIOFLOAT, samples[ MAXBUFFERSIZE * MAXCHANNELS ], RADFFT_ALIGN );
  RAD_ALIGN( AUDIOFLOAT, loadbuf[ MAXBUFFERSIZE ], RADFFT_ALIGN );

  #endif

  rrassert( ( transform_size * chans ) <= ( MAXBUFFERSIZE * MAXCHANNELS ) );

  lossy = (AUDIOFLOAT)lossy_level;
  
  VarBitsOpen( vb, buf );

  //if ( flags & BINKACNEWFORMAT )
  {
    VarBitsPut( vb, 0, 2 );
  }

  for ( i = 0 ; i < chans ; i++ )
  {
    U32 t;

    // Load up the data structure with the samples
    load_samples( loadbuf,
                  in_data,
                  in_bytes,
                  i,
                  chans,
                  transform_size );

    // round the samples on the ends
    ramp_samples( loadbuf, 0.0f, 1.0f, ( transform_size / ( WINDOWRATIO * 2 ) ) );
    ramp_samples( loadbuf + transform_size - ( transform_size / ( WINDOWRATIO * 2 ) ),
                  1.0f, 0.0f, ( transform_size / ( WINDOWRATIO * 2 ) ) );

    // do the fft
    radfft_rfft( (rfft_complex*)samples, loadbuf, transform_size );

    // Normalize the coeffs
    multiply_samples_by_scalar( samples, transform_size, transform_size_root );

    // Do perceptual model
    do_perceptual( threshold, lossy, threshold_curve_adj, samples, transform_size, last_pow_pha, bands, num_bands );

    // Load up the data structure with the samples (again)
    load_samples( loadbuf,
                  in_data,
                  in_bytes,
                  i,
                  chans,
                  transform_size );

    // do the dct
    //if ( flags & BINKACNEWFORMAT )
    {
      radfft_dct( samples, loadbuf, transform_size );
      samples[ 0 ] *= 0.5F;
    }

    // Normalize the coeffs
    multiply_samples_by_scalar( samples, transform_size, transform_size_root );

    // figure out the best threshold for each band
    calc_best_quant( best_qlevel, samples, threshold, bands, num_bands );

    // check any band level is below the band's ath and crank if up if necessary
    clamp_best_quant_to_ath( best_qlevel, band_ath, num_bands, lossy );


    // dump out the DC component and the Nyquist frequency in high resolution
    t = ftofxp( samples[ 0 ] );
    VarBitsPut( vb, t, FXPBITS );
    t = ftofxp( samples[ 1 ] );
    VarBitsPut( vb, t, FXPBITS );

    // output the thresholds
    for ( t = 0 ; t < num_bands ; t++ )
    {
      if (flags & BINKAC20)
      {
        U8 qlevel = (U8)best_qlevel[t];
        if (qlevel > 95)
            qlevel = 95;
        VarBitsPut(vb, qlevel, 7);
      }
      else
      {
        VarBitsPut(vb, (U8)best_qlevel[t], 8);
      }
    }

    // encode a channel
    encode_one_channel( transform_size,
                        &vb,
                        samples,
                        best_qlevel,
                        num_bands,
                        bands,
                        ath,
                        lossy,
                        flags & BINKAC20);
  }

  #ifdef DEBUGSTACKVARS
  popfree( samples );
  #endif

  VarBitsPutAlign( vb );
  return( ( VarBitsSize( vb ) / 8 ) );
}

RADDEFFUNC HBINKAUDIOCOMP RADLINK BinkAudioCompressOpen( U32 rate, U32 chans, U32 flags, BinkAudioCompressAllocFnType* memalloc, BinkAudioCompressFreeFnType* memfree)
{
  U32 i, j;
  U32 transform_size, transform_size_half, buffer_size;
  U32 num_bands;
  S32 nyq;
  AUDIOFLOAT adj_for_old;
  AUDIOFLOAT threshold_curve_adj;

  HBINKAUDIOCOMP ba;

  if ( rate >= 44100 )
    transform_size = 2048;
  else if ( rate >= 22050 )
    transform_size = 1024;
  else
    transform_size = 512;

  // in bytes
  buffer_size = transform_size * chans * 2;

  // by default, no adjustment for old audio codec
  adj_for_old = 1.0f;
  threshold_curve_adj = 12.0f;

  transform_size_half = transform_size / 2;
  nyq = ( rate + 1 ) / 2;

  // calculate the number of bands we'll use
  for( i = 0 ; i < TOTBANDS ; i++ )
  {
    if ( bandtopfreq[ i ] >= (U32) nyq )
      break;
  }
  num_bands = i;

  // allocate our memory
  {
    U32 * bands_ptr;
    AUDIOFLOAT * renorm_ptr;
    AUDIOFLOAT * ath_ptr;
    AUDIOFLOAT * band_ath_ptr;
    AUDIOFLOAT * last_pow_pha_ptr;
    S16 * in_ptr;
    S16* out_ptr;
    char pmbuf[ PushMallocBytesForXPtrs( 16 ) ];
  
    pushmallocinit( pmbuf, 16 );
    pushmalloc( pmbuf, &bands_ptr, 4 * ( num_bands + 1 ) );
    pushmalloc( pmbuf, &renorm_ptr, 4 * num_bands );
    pushmalloc( pmbuf, &last_pow_pha_ptr, 4 * 2 * 2 * chans * transform_size_half ); // 2 for power/phase, the 2 for two frames worth
    pushmalloc( pmbuf, &in_ptr, buffer_size );
    pushmalloc( pmbuf, &ath_ptr, 4 * transform_size );
    pushmalloc( pmbuf, &band_ath_ptr, 4 * num_bands );
    pushmalloc( pmbuf, &out_ptr, buffer_size + ( buffer_size / 2 ) );

    ba = (HBINKAUDIOCOMP) popmalloc( pmbuf, sizeof( BINKAUDIOCOMP ), memalloc );
    if ( ba == 0)
      return( 0 );

    rrmemsetzero( ba, sizeof( BINKAUDIOCOMP ) );

    ba->bands = bands_ptr;
    ba->renorm = renorm_ptr;
    ba->last_pow_pha = last_pow_pha_ptr;
    memset(ba->last_pow_pha, 0, 4 * 2 * 2 * chans * transform_size_half);
    ba->inp = in_ptr;
    ba->outp = out_ptr;
    ba->ath = ath_ptr;
    ba->band_ath = band_ath_ptr;
    ba->memalloc = memalloc;
    ba->memfree = memfree;

    radfft_init();
  }


  ba->flags = flags;
  ba->chans = chans;
  ba->transform_size = transform_size;

  ba->buffer_size = buffer_size;
  ba->window_size = buffer_size / WINDOWRATIO;

  ba->num_bands = num_bands;

  ba->transform_size_root = ( 1.0F / ( (AUDIOFLOAT) radfsqrt( ba->transform_size ) ) );

  // calculate the band ranges
  for( i = 0 ; i < num_bands ; i++ )
  {
    ba->bands[ i ] = ( bandtopfreq[ i ] * transform_size_half ) / nyq;
  }
  ba->bands[ i ] = transform_size_half;

  rrmemsetzero( ba->renorm, ba->num_bands * sizeof( ba->renorm[ 0 ] ) );
  for( i = 0 ; i < ba->num_bands ; i++ )
  {
    for( j = 0 ; j < ba->num_bands ; j++ )
      ba->renorm[ j ] += (AUDIOFLOAT) spreadfactors[ (S32) j - (S32) i + TOTBANDS ];
  }

  for( i = 0 ; i < ba->num_bands ; i++ )
  {
    ba->renorm[ i ] = (AUDIOFLOAT) ( 1.0 / (F64) ba->renorm[ i ] );
  }

  // initialize the ath
  for( i = 0 ; i < transform_size ; i++ )
  {
    AUDIOFLOAT ath, f;

    f = ( (AUDIOFLOAT) mult64anddiv( i, nyq, transform_size ) ) / 1000.0f;

    f = f * adj_for_old;

    ath = 3.64f * (AUDIOFLOAT) radpow( f, -0.8 );
    ath += ( -6.5f * (AUDIOFLOAT) radexp( -0.6f * ( f - 3.3f ) * ( f - 3.3f ) ) );
    ath += ( 0.001f * f * f * f * f );

    ba->ath[ i ] = ath;
  }

  // get the lowest point on the curve for each band
  for( i = 0 ; i < ba->num_bands ; i++ )
  {
    AUDIOFLOAT low = 1000.0f;

    for( j = ( ba->bands[ i ] * 2 ) ; j < ( ba->bands[ i + 1 ] * 2 ) ; j++ )
    {
      AUDIOFLOAT ath, f;

      f = ( (AUDIOFLOAT) mult64anddiv( j, nyq, transform_size ) ) / 1000.0f;

      ath = ba->ath[ j ];

      if ( ath < low )
        low = ath;
    }

    ba->band_ath[ i ] = low;
  }

  ba->threshold_curve_adj = threshold_curve_adj;

  ba->start_frame = 1;

  return( ba );
}


RADDEFFUNC void RADLINK BinkAudioCompressLock( HBINKAUDIOCOMP ba,
                                               void**ptr,
                                               U32*len )
{
  if ( ba->start_frame )
  {
    if ( ptr )
      *ptr = ba->inp;
    if ( len )
      *len = ba->buffer_size;
  }
  else
  {
    if ( ptr )
      *ptr = ( (U8*) ba->inp ) + ba->window_size;
    if ( len )
      *len = ( ba->buffer_size - ba->window_size );
  }
}


RADDEFFUNC void RADLINK BinkAudioCompressUnlock( HBINKAUDIOCOMP ba,
                                                 U32 lossylevel,
                                                 U32 filled,
                                                 void** output,
                                                 U32* outbytes,
                                                 U32* uncompressedbytesused )
{
  U32 in_bytes, out_bytes, used_bytes;

  ba->got += filled;

  in_bytes = filled;

  if ( ba->start_frame )
    ba->start_frame = 0;
  else
    in_bytes += ba->window_size;

  out_bytes = Percept( ba->transform_size,
                       ba->transform_size_root,
                       ba->chans,
                       ba->flags,
                       ba->last_pow_pha,
                       lossylevel,
                       ba->threshold_curve_adj,
                       ba->outp,
                       ba->inp,
                       in_bytes,
                       ba->num_bands,
                       ba->bands,
                       ba->band_ath,
                       ba->ath,
                       ba->renorm );

  // Store end of buffer
  rrmemmovebig( ba->inp,
                ( (U8*) ba->inp ) + ( ba->buffer_size - ba->window_size ),
                ba->window_size );

  // set the output values
  if ( output )
    *output = ba->outp;
  if ( outbytes )
    *outbytes = out_bytes;

  rrassert( out_bytes <= (ba->buffer_size +(ba->buffer_size/2)) );

  // calculate how many bytes of audio we used this call
  used_bytes = ba->buffer_size - ba->window_size;
  if ( ( used_bytes + ba->given ) > ba->got )
    used_bytes = ba->got - ba->given;
  ba->given += used_bytes;

  if ( uncompressedbytesused )
    *uncompressedbytesused = used_bytes;
}


RADDEFFUNC void RADLINK BinkAudioCompressClose(HBINKAUDIOCOMP ba)
{
  popfree( ba, ba->memfree );
}

