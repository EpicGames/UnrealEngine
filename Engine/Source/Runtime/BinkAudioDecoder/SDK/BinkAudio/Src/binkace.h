// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef __BINKACEH__
#define __BINKACEH__

#ifndef __RADRR_COREH__
  #include "rrCore.h"
#endif

#define AUDIOFLOAT F32

#ifdef WRAP_PUBLICS
#define rfmerge3(name,add) name##add
#define rfmerge2(name,add) rfmerge3(name,add)
#define rfmerge(name)      rfmerge2(name,WRAP_PUBLICS)
#define BinkAudioCompressOpen                          rfmerge(BinkAudioCompressOpen)
#define BinkAudioCompressLock                          rfmerge(BinkAudioCompressLock)
#define BinkAudioCompressUnlock                        rfmerge(BinkAudioCompressUnlock)
#define BinkAudioCompressClose                         rfmerge(BinkAudioCompressClose)
#endif

//===========================================
//  encoding API
//===========================================

struct BINKAUDIOCOMP;
typedef struct BINKAUDIOCOMP * HBINKAUDIOCOMP;

typedef void* BinkAudioCompressAllocFnType(UINTa ByteCount);
typedef void BinkAudioCompressFreeFnType(void* Ptr);

//#define BINKACNEWFORMAT 1
#define BINKAC20 4 // if set, BINKACNEWFORMAT is assumed

RADDEFFUNC HBINKAUDIOCOMP RADLINK BinkAudioCompressOpen(U32 rate,U32 chans, U32 flags, BinkAudioCompressAllocFnType* memalloc, BinkAudioCompressFreeFnType* memfree);
RADDEFFUNC void RADLINK BinkAudioCompressLock(HBINKAUDIOCOMP ba,void**ptr, U32*len);
RADDEFFUNC void RADLINK BinkAudioCompressUnlock(HBINKAUDIOCOMP ba, U32 lossylevel, U32 filled, void** output,U32* outbytes, U32* uncompressedbytesused);
RADDEFFUNC void RADLINK BinkAudioCompressClose(HBINKAUDIOCOMP ba);

#endif