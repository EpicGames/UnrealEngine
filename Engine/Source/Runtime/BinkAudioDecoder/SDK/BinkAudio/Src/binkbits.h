// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef __RADRR_COREH__
  #include "rrCore.h"
#endif

#if !defined(__RAD64__)  // 32-bit path

typedef struct BINKVARBITS { void * cur; U32 bits; U32 bitlen; } BINKVARBITS;

#define BINKBITSLOCALS( name ) void * name##cur; U32 name##bits; U32 name##bitlen
#define BinkVarBitsOpen(vb,pointer) { (vb).bits=BINK_LOAD32(pointer); (vb).cur=((char*)pointer)+4; (vb).bitlen = 32; }

// only on big endian, does this do anything (it's for when you don't preflip the data)
#if defined(__RADEMSCRIPTEN__)
#include <emscripten.h>
#define BINK_LOAD32(ptr) ((U32)*((emscripten_align1_int*)(ptr)))
#else
#if defined(BINKAUDIODATALE)
#define BINK_LOAD32(ptr) RR_GET32_LE(ptr)
#else
#define BINK_LOAD32(ptr) RR_GET32_NATIVE(ptr)
#endif
#endif

#define BinkBitsGet(v,typ,vb,len,mask)                          \
{                                                               \
  if ((vb##bitlen)<(len)) {                                     \
    register U32 nb=BINK_LOAD32((U32* RADRESTRICT)(vb##cur));   \
    v=(typ)(((vb##bits)|(nb<<(vb##bitlen)))&(mask));            \
    (vb##bits)=nb>>((len)-(vb##bitlen));                        \
    (vb##bitlen)=(vb##bitlen)+32-(len);                         \
    (vb##cur)=((char*)(vb##cur))+4;                             \
  } else {                                                      \
    v=(typ)((vb##bits)&(mask));                                 \
    (vb##bits)>>=(len);                                         \
    (vb##bitlen)-=(len);                                        \
  }                                                             \
}

#if defined(__RADLITTLEENDIAN__) 

#define MAX_AT_LEAST_BITS 25

// load as many bytes as we can each load on little endian (unaligned loads)
#define BinkBitsAtLeastStart( vb,len )                          \
{                                                               \
  if ((vb##bitlen)<(len)) {                                     \
    U32 bl=((32-(vb##bitlen))>>3);                              \
    U32 nb=BINK_LOAD32((U32* RADRESTRICT)(vb##cur));            \
    (vb##bits)=((vb##bits)|(nb<<(vb##bitlen)));                 \
    (vb##cur)=((char*)(vb##cur))+bl;                            \
    (vb##bitlen)=(vb##bitlen)+(bl<<3);                          \
  }                                                             \
}

#define BinkBitsAtLeastEnd(vb) 

#else

#define MAX_AT_LEAST_BITS 32

// when running on big endian, we can load 32-bits at a time, but only
//   advance in the End state, not the start - this keeps us aligned
#define BinkBitsAtLeastStart( vb,len )                          \
{                                                               \
  if ((vb##bitlen)<(len))                                       \
  {                                                             \
    U32 nb=BINK_LOAD32((U32* RADRESTRICT)(vb##cur));            \
    (vb##bits)=((vb##bits)|(nb<<(vb##bitlen)));                 \
  }                                                             \
}

#define BinkBitsAtLeastEnd(vb)                                                   \
{                                                                                \
  if ( ( (S32)(vb##bitlen) ) <= 0 )                                              \
  {                                                                              \
    (vb##bits)=BINK_LOAD32(((U32* RADRESTRICT)(vb##cur)))>>(-(S32)(vb##bitlen)); \
    (vb##cur)=((char*)(vb##cur))+4;                                              \
    (vb##bitlen)=(vb##bitlen)+32;                                                \
  }                                                                              \
}

#endif

#else

typedef struct BINKVARBITS { void * cur; U64 bits; U32 bitlen; } BINKVARBITS;

#define BINKBITSLOCALS( name ) void * name##cur; U64 name##bits; U32 name##bitlen
#define BinkVarBitsOpen(vb,pointer) { (vb).bits=*(U64*)(pointer); (vb).cur=((char*)pointer)+8; (vb).bitlen = 64; }

#define BINK_LOAD32 RR_GET32_NATIVE

#define BinkBitsGet(v,typ,vb,len,mask)                          \
{                                                               \
  if ((vb##bitlen)<(len)) {                                     \
    U64 nb=*((U64* RADRESTRICT)(vb##cur));             \
    v=(typ)(((vb##bits)|(nb<<(vb##bitlen)))&(mask));            \
    (vb##bits)=nb>>((len)-(vb##bitlen));                        \
    (vb##bitlen)=(vb##bitlen)+64-(len);                         \
    (vb##cur)=((char*)(vb##cur))+8;                             \
  } else {                                                      \
    v=(typ)((vb##bits)&(mask));                                 \
    (vb##bits)>>=(len);                                         \
    (vb##bitlen)-=(len);                                        \
  }                                                             \
}

#define MAX_AT_LEAST_BITS 57

#define BinkBitsAtLeastStart( vb,len )                          \
{                                                               \
  if ((vb##bitlen)<(len))                                       \
  {                                                             \
    U32 bl=((64-(vb##bitlen))>>3);                              \
    U64 nb=*((U64* RADRESTRICT)(vb##cur));                      \
    (vb##bits)=((vb##bits)|(nb<<(vb##bitlen)));                 \
    (vb##cur)=((char*)(vb##cur))+bl;                            \
    (vb##bitlen)=(vb##bitlen)+(bl<<3);                          \
  }                                                             \
}

#define BinkBitsAtLeastEnd( vb ) 

#define VarBits32Use(vb,len) { (vb).bits >>= (len); (vb).bitlen -= (len); }

#define MAX_PEEK 64

#endif

#define BINKBITSCOPY(name, from) { name##cur = from##cur; name##bits = from##bits; name##bitlen = from##bitlen; }

#define BinkBitsInAtLeastPeek(vb) ( vb##bits )
#define BinkBitsInAtLeastUse( vb, bl ) { (vb##bits) >>= (bl); (vb##bitlen) -= (bl); }

#define BinkBitsPeek(v, typ, vb, len)                           \
{                                                               \
  BinkBitsAtLeastStart( vb, len )                               \
  (v)=(typ)BinkBitsInAtLeastPeek( vb );                         \
}

#define BinkBitsUse( vb, bl ) { BinkBitsInAtLeastUse( vb, bl ); BinkBitsAtLeastEnd( vb ); }

#define BinkVarBitsUse(vb,len) { (vb).bits >>= (len); (vb).bitlen -= (len); }

#define VarBitsCopyToBinkBits( local, vb ) local##cur = (vb).cur; local##bits = (vb).bits; local##bitlen = (vb).bitlen; 
#define BinkBitsCopyToVarBits( vb, local ) { (vb).cur = local##cur;  (vb).bits = local##bits;  (vb).bitlen = local##bitlen; }

#define BinkBitsSizeBytesRoundedToU32( local, base ) ((((((U8*)(local##cur))-((U8*)base))-(local##bitlen/8))+3)&~3)
#define BinkVarBitsSizeBytesRoundedToU32( vb, base ) ((((((U8*)((vb).cur))-((U8*)base))-((vb).bitlen/8))+3)&~3)
