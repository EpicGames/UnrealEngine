// Copyright Epic Games, Inc. All Rights Reserved.

#include "egttypes.h"

//
// All users of the console abstraction need to use
// CONSOLE_MAIN as the main definition.
// eg
// CONSOLE_MAIN
// {
//      console_init();
//      // do stuff
//      for (;;)
//      {
//          console_handleevents();
//          // do stuff
//      }
//      return 1;
// }


#ifdef __RAD_NDA_PLATFORM__

    #include RR_PLATFORM_PATH_STR( __RAD_NDA_PLATFORM__, _console.h )

#else

    #if defined(__RADANDROID__)
        #include <android/log.h>
        #include <android_native_app_glue.h>
        
        static void console_printf(const char *fmt, ...)
        {
            va_list arg;
            va_start(arg, fmt);
            __android_log_vprint(ANDROID_LOG_INFO, "ba_stdout", fmt, arg);
            va_end(arg);
        }

        static void console_init() {}
        static void console_handleevents() {}
        #define CONSOLE_MAIN extern "C" int main(int argc, char** argv)
        
    #elif defined(__RADNT__) || defined(__RADWINRT__)
        #include <conio.h>
        #include <windows.h>
        #include <stdio.h>
        static void console_init() {}
        static void console_handleevents() {}
        #define CONSOLE_MAIN int main(int argc, char** argv)
        static void console_printf(char const* fmt, ...)
        {
            va_list args;
            va_start(args, fmt);
            vprintf(fmt, args);

            // this flush is here so that when we run from a parent exe, we don't get
            // hit by the buffering behavior that the vc crt adds if the filetype of
            // stdout is a pipe. We print rarely enough that the perf shouldn't matter.
            fflush(stdout);

            va_end(args);
        }

        #if defined(_MSC_VER)
        #pragma warning(disable: 4505) // unreferenced static function
        #endif

    #elif defined(__RADLINUX__) || defined(__RADMAC__)
        #include <stdio.h>
        #include <termios.h>
        #include <unistd.h>
        #include <stdlib.h>

        static struct termios initial_settings, new_settings;
        static int peek_character =  -1;

        int _kbhit();
        int _kbhit()
        {
            unsigned char ch;
            int nread;
            if  (peek_character != -1) 
                return 1;

            new_settings.c_cc[VMIN]=0;
            tcsetattr(0,  TCSANOW, &new_settings);
            nread = (int)read(0,&ch,1);
            new_settings.c_cc[VMIN]=1;
            tcsetattr(0,  TCSANOW, &new_settings);
            if(nread == 1)
            {
                peek_character = ch;
                return 1;
            }
            return  0;
        }

        int _getch();
        int _getch()
        {
            unsigned char ch;
            int i;
            if(peek_character != -1)
            {
                ch = (unsigned char)peek_character;
                peek_character = -1;
                return ch;
            }
            i=(int)read(0,&ch,1);
            if (i!=1) 
                ch = 0;
            return ch;
        }

        static void console_cleanup()
        {
            tcsetattr(0,  TCSANOW, &initial_settings);
        }
        static void console_init() 
        {
            setbuf( stdout, 0 );
            tcgetattr(0,&initial_settings);
            new_settings =  initial_settings;
            new_settings.c_lflag &= ~ICANON;
            new_settings.c_lflag &= ~ECHO;
            new_settings.c_iflag &= ~ICRNL;
            new_settings.c_cc[VMIN] = 1;
            new_settings.c_cc[VTIME] = 0;
            tcsetattr(0,  TCSANOW, &new_settings);
            atexit(console_cleanup);
        }
        static void console_handleevents() {}
        #define CONSOLE_MAIN int main(int argc, char** argv)
        #define console_printf printf

    #endif // public platform
#endif // nda platform