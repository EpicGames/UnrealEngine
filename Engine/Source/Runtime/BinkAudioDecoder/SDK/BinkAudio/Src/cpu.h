// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef __CPUH__
#define __CPUH__

// cpu_check( force_use, force_no_use ) - checks features and then forces some on and off
// cpu_can_use( feature ) - can use all of these features? (call cpu_check once somewhere first)
// cpu_can_use_any( feature ) - can use any of these features? (call cpu_check once somewhere first)
// cpu_clean_up() - just an emms for mmx, if you want to wait until a bunch of mmx calls are made
// cpu_features() - just returns a bit field of features


#ifndef __RADRR_CORE2H__
#include "rrCore.h"
#endif

#ifdef __RADX86__

#define CPU_MMX   1
#define CPU_3DNOW 2
#define CPU_SSE   4
#define CPU_SSE2  8
#define CPU_SSE3  16
#define CPU_SSSE3 32

#elif defined( __RADALTIVEC__ )

#define CPU_ALTIVEC 1

#elif defined( __RADARM__ )

#define CPU_NEON 1

#define CPU_check( u, du )
#define CPU_clean_up()

#ifdef __RADNEON__
#define CPU_can_use( has ) ( ( has & CPU_NEON ) == has )
#define CPU_can_use_any( has ) ( has & CPU_NEON )
#define CPU_features() ( CPU_NEON ) 
#define CPU_ALWAYS_NEON
#else
#define CPU_can_use( has ) 0
#define CPU_can_use_any( has ) 0
#define CPU_features() 0
#endif

#endif


#if defined( __RAD_NDA_PLATFORM__ )

#include RR_PLATFORM_PATH_STR( __RAD_NDA_PLATFORM__, _cpu.h )

#elif defined( __RADX86__ )

#ifdef WRAP_PUBLICS
#define rfmerge3(name,add) name##add
#define rfmerge2(name,add) rfmerge3(name,add)
#define rfmerge(name)      rfmerge2(name,WRAP_PUBLICS)
#define cpu_features_to_use             rfmerge(cpu_features_to_use)
#define cpu_features_avail              rfmerge(cpu_features_avail)
#define CPU_check                       rfmerge(CPU_check)
#define CPU_clear                       rfmerge(CPU_clear)
#endif


RADDECLAREDATA S32 cpu_features_to_use;
RADDECLAREDATA S32 cpu_features_avail;

#define CPU_can_use( has ) ( ( cpu_features_to_use & has ) == has )
#define CPU_can_use_any( has ) ( cpu_features_to_use & has )
#define CPU_features() ( cpu_features_to_use )

RADDEFFUNC void CPU_check( S32 use, S32 dont_use );

RADDEFFUNC void CPU_clear( void );

#ifdef __RAD64__

#define CPU_clean_up()    \
{                         \
  if ( CPU_can_use( CPU_MMX ) )   \
  {                       \
    CPU_clear();          \
  }                       \
}

#else

#include <mmintrin.h>

#define CPU_clean_up()    \
{                         \
  if ( CPU_can_use( CPU_MMX ) )   \
  {                       \
    _mm_empty();          \
  }                       \
}

#endif

#elif defined( __RADARM__ ) 

#elif defined( __RADEMSCRIPTEN__ )

#define CPU_check( u, du )
#define CPU_clean_up()
#define CPU_can_use( has ) 1
#define CPU_can_use_any( has ) 1
#define CPU_features() 0

#else

#error "CPU platform error."

#endif


#endif
