// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef DO_SCALAR
#undef DO_SCALAR
#endif

#if defined(__RADX86__) 

// for sse intrinsics.
#include <emmintrin.h>
#include <mmintrin.h>

#if defined (__RAD64__) || (_M_IX86_FP>=2)  //win64 or arch:SSE2 specified
    #define CallCrossFade CrossFadeSSE2
#else
    // x86 32-bit try sse or fallback to mmx
    #define CallCrossFade CrossFadeDispatch

    static void CrossFadeMMX(S16* outp,S16* overlap, U32 windowsize, S32 windowshift)
    {
        U32 i;
        {
            __m64 IVec = _mm_setr_pi16(0, 1, 2, 3);
            __m64 IncVec = _mm_set1_pi16(4);
            __m64 ShiftVec = _mm_set_pi32(0, windowshift);
            __m64 ZeroVec = _mm_setzero_si64();
            __m64 ShiftFor32Bit = _mm_set_pi32(0, 16);

            // linearly blend the sample over the window size
            for ( i = 0; i < windowsize ; i+=8 )
            {
                __m64 OutpVec = *(__m64*)((char*)outp + i);
                __m64 OverlapVec = *(__m64*)((char*)overlap + i);
                __m64 OutpVecLow, OutpVecHigh;
                __m64 Temp;
                __m64 OverlapHigh, OverlapLow;
                
                // outp - overlap
                OutpVec = _mm_sub_pi16(OutpVec, OverlapVec);

                // *i
                OutpVecHigh = _mm_mulhi_pi16(OutpVec, IVec);
                OutpVecLow = _mm_mullo_pi16(OutpVec, IVec);

                // move to something useful.
                Temp = _mm_unpackhi_pi16(OutpVecLow, OutpVecHigh);
                OutpVecLow = _mm_unpacklo_pi16(OutpVecLow, OutpVecHigh);

                // /ws
                OutpVecHigh = _mm_sra_pi32(Temp, ShiftVec);
                OutpVecLow = _mm_sra_pi32(OutpVecLow, ShiftVec);

                // overlap -> 32 bit
                OverlapHigh = _mm_unpackhi_pi16(ZeroVec, OverlapVec);
                OverlapLow = _mm_unpacklo_pi16(ZeroVec, OverlapVec);

                OverlapHigh = _mm_sra_pi32(OverlapHigh, ShiftFor32Bit);
                OverlapLow = _mm_sra_pi32(OverlapLow, ShiftFor32Bit);

                // Add overlap
                OutpVecLow = _mm_add_pi32(OutpVecLow, OverlapLow);
                OutpVecHigh = _mm_add_pi32(OutpVecHigh, OverlapHigh);

                // back to 16 bit
                OutpVec = _mm_packs_pi32(OutpVecLow, OutpVecHigh);

                // inc i.
                IVec = _mm_add_pi16(IVec, IncVec);


                *(__m64*)((char*)outp + i) =  OutpVec;
            }
            _mm_empty();
        } 
    } 

    static void CrossFadeSSE2(S16* outp,S16* overlap, U32 windowsize, S32 windowshift);
    static void CrossFadeDispatch(S16* outp,S16* overlap, U32 windowsize, S32 windowshift)
    {
        if (CPU_can_use(CPU_SSE2))
        {
            CrossFadeSSE2(outp,overlap,windowsize,windowshift);
            return;
        }

        CrossFadeMMX(outp,overlap,windowsize,windowshift);
        return;
    }

#endif

static void CrossFadeSSE2(S16* outp,S16* overlap, U32 windowsize, S32 windowshift)
{
    U32 i;
    {
        __m128i IVec = _mm_setr_epi16(0, 1, 2, 3, 4, 5, 6, 7);
        __m128i IncVec = _mm_set1_epi16(8);
        __m128i ShiftVec = _mm_set_epi32(0, 0, 0, windowshift);
        __m128i SizeVec = _mm_set1_epi16((U16)windowsize >> 1);

        // linearly blend the sample over the window size
        for ( i = 0; i < windowsize ; i+=16 )
        {
            __m128i OutpVec = _mm_load_si128((__m128i*)((char*)outp + i));
            __m128i OverlapVec = _mm_load_si128((__m128i*)((char*)overlap + i));

            __m128i OverlapScale = _mm_sub_epi16(SizeVec, IVec);

            // Outp * i
            __m128i OutpLow = _mm_mullo_epi16(OutpVec, IVec);
            __m128i OutpHigh = _mm_mulhi_epi16(OutpVec, IVec);

            // Overlap * (ws - i)
            __m128i OverlapLow = _mm_mullo_epi16(OverlapVec, OverlapScale);
            __m128i OverlapHigh = _mm_mulhi_epi16(OverlapVec, OverlapScale);

            // Interleave in to 32 bit values.
            __m128i Temp = _mm_unpackhi_epi16(OutpLow, OutpHigh);
            OutpLow = _mm_unpacklo_epi16(OutpLow, OutpHigh);
            OutpHigh = Temp;

            Temp = _mm_unpackhi_epi16(OverlapLow, OverlapHigh);
            OverlapLow = _mm_unpacklo_epi16(OverlapLow, OverlapHigh);
            OverlapHigh = Temp;

            // (outp[i] * i + overlap[i] * (ws - i))
            OverlapLow = _mm_add_epi32(OverlapLow, OutpLow);
            OverlapHigh = _mm_add_epi32(OverlapHigh, OutpHigh);

            // / ws
            OverlapLow = _mm_sra_epi32(OverlapLow, ShiftVec);
            OverlapHigh = _mm_sra_epi32(OverlapHigh, ShiftVec);

            // -> 16 bit
            OutpVec = _mm_packs_epi32(OverlapLow, OverlapHigh);

            IVec = _mm_add_epi16(IVec, IncVec);

            _mm_store_si128((__m128i*)((char*)outp + i), OutpVec);
        }
    } 
}

#elif defined(__RADNEON__)

#include <arm_neon.h>

#define CallCrossFade CrossFadeNeon

static RAD_ALIGN( S16, starting_blend[ 8 ], 16 ) = { 0, 1, 2, 3, 4, 5, 6, 7 };

static void CrossFadeNeon(S16* outp,S16* overlap, U32 windowsize, S32 windowshift)
{
	int16x8_t IVec = *(int16x8_t*)starting_blend;
	int16x8_t IncVec = vmovq_n_s16(8);
	int32x4_t ShiftVec = vmovq_n_s32(-windowshift);
	int16x8_t SizeVec = vmovq_n_s16((S16)(windowsize >> 1));

	// linearly blend the sample over the window size
	for ( U32 i = 0; i < windowsize ; i += 16 )
	{
		int16x8_t OutpVec = vld1q_s16((S16*)((char*)outp + i));
		int16x8_t OverlapVec = vld1q_s16((S16*)((char*)overlap + i));

		int16x8_t OverlapScale = vsubq_s16(SizeVec, IVec);

		// Outp * i
		int32x4_t OutpLow = vmull_s16(vget_low_s16(OutpVec), vget_low_s16(IVec));
		int32x4_t OutpHigh = vmull_s16(vget_high_s16(OutpVec), vget_high_s16(IVec));

		// Overlap * (ws - i)
		int32x4_t OverlapLow = vmull_s16(vget_low_s16(OverlapVec), vget_low_s16(OverlapScale));
		int32x4_t OverlapHigh = vmull_s16(vget_high_s16(OverlapVec), vget_high_s16(OverlapScale));

		// (outp[i] * i + overlap[i] * (ws - i))
		OverlapLow = vaddq_s32(OverlapLow, OutpLow);
		OverlapHigh = vaddq_s32(OverlapHigh, OutpHigh);

		// / ws
		OverlapLow = vshlq_s32(OverlapLow, ShiftVec);
		OverlapHigh = vshlq_s32(OverlapHigh, ShiftVec);

		// -> 16 bit
		OutpVec = vcombine_s16(vmovn_s32(OverlapLow), vmovn_s32(OverlapHigh));

		IVec = vaddq_s16(IVec, IncVec);

		vst1q_s16((S16*)((char*)outp + i), OutpVec);
	}
}

#elif defined(__RADBIGENDIAN__)

#include RR_PLATFORM_PATH_STR( __RAD_NDA_PLATFORM__, _crsfade.inl )

#elif defined( __RADEMSCRIPTEN__ )

// this is just emscripten currently
#define CallCrossFade CrossFade
static void CrossFade(S16* outp,S16* overlap, U32 windowsize, S32 windowshift)
{
    U32 i;
    U32 ws = windowsize / 2;  // convert from bytes to samples
    // bi-linearly blend the sample over the window size
    for ( i = 0; i < ws ; i++ )
    {
        outp[i] = (S16) (((((S32)outp[i] - (S32)overlap[i]) * i) >> windowshift ) + (S32) overlap[i] );
    }
}

#else

#error crsfade.inl platform

#endif
