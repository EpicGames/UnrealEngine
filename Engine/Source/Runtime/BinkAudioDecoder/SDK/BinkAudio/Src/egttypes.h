// Copyright Epic Games, Inc. All Rights Reserved.

#if !defined(__EGTTYPESH__) && !defined(__RADTYPESH__) && !defined(__RADRR_COREH__) 
#define __RADTYPESH__
#define __EGTTYPESH__
#define __RADRR_COREH__ // block old rr_core

#if !defined(__RADRES__) // don't include anything for resource compiles

//  __RAD32__ means at least 32 bit code (always defined)
//  __RAD64__ means 64 bit addressing (64-bit OSes only) (__RAD64REGS__ will be set)
//  __RAD64REGS__ means 64 bit registers

//  __RADNT__ means Win32 and Win64 desktop
//  __RADWINRT__ means Windows Store/Phone App (x86, x64, arm)
//  __RADWIN__ means win32, win64, windows store/phone, consoles
//  __RADWINRTAPI__ means Windows RT API (Win Store, Win Phone, consoles)
//  __RADMAC__ means MacOS (32 or 64-bit)
//  __RADANDROID__ means Android NDK
//  __RADLINUX__ means Linux (32 or 64-bit)
//  __RADQNX__ means QNX
//  __RADIPHONE__ means iphone (ios, tvos, or watchos)
//  __RADIOS__ means iOS (+ __RADIPHONE__ will also be set)
//  __RADTVOS__ means tvOS (+ __RADIPHONE__ will also be set)
//  __RADWATCHOS__ means watchOS (+ __RADIPHONE__ will also be set)
//  __RADEMSCRIPTEN__ means Emscripten

//  __RADARM__ means arm
//  __RADPPC__ means powerpc
//  __RADX86__ means x86 or x64
//  __RADX64__ means x64  (+ __RADX86__ will also be set)
//  __RADNEON__ means you can use NEON intrinsics on ARM
//  __RADSSE2__ means you can use SSE2 intrinsics on X86

// __RADNOVARARGMACROS__ means #defines can't use ...

// RADDEFSTART is "extern "C" {" on C++, nothing on C
// RADDEFEND is "}" on C++, nothing on C

// RADEXPFUNC and RADEXPLINK are used on both the declaration
//   and definition of exported functions.
//    RADEXPFUNC int RADEXPLINK exported_func()

// RADDEFFUNC and RADLINK are used on both the declaration
//   and definition of public functions (but not exported).
//    RADDEFFUNC int RADLINK public_c_func()

// RADRESTRICT is for non-aliasing pointers/

// RADSTRUCT is defined as "struct" on msvc, and
//   "struct __attribute__((__packed__))" on gcc/clang
//   Used to sort of address generic structure packing
//   (we still require #pragma packs to fix the
//   packing on windows, though)


// ========================================================
// C++ name demangling nonsense

#ifdef __cplusplus
  #define RADDEFFUNC extern "C"
  #define RADDEFSTART extern "C" {
  #define RADDEFEND }
  #define RADDEFAULT( val ) =val
#else
  #define RADDEFFUNC
  #define RADDEFSTART
  #define RADDEFEND
  #define RADDEFAULT( val )
#endif

//===========================================================================
// preprocessor string stuff
#define RR_STRINGIZE( L )         #L
#define RR_DO_MACRO( M, X )       M(X)
#define RR_STRINGIZE_DELAY( X )   RR_DO_MACRO( RR_STRINGIZE, X )

#define RR_STRING_JOIN(arg1, arg2)              RR_STRING_JOIN_DELAY(arg1, arg2)
#define RR_STRING_JOIN_DELAY(arg1, arg2)        RR_STRING_JOIN_IMMEDIATE(arg1, arg2)
#define RR_STRING_JOIN_IMMEDIATE(arg1, arg2)    arg1 ## arg2

// makes "plat/plat_file"
#define RR_PLATFORM_PATH_STR( plat, file ) RR_STRINGIZE_DELAY( RR_STRING_JOIN( plat, file ) )


// ========================================================
// First off, we detect your platform

#ifdef __RAD_NDA_PLATFORM__

  #include RR_PLATFORM_PATH_STR( __RAD_NDA_PLATFORM__, _egttypes.h )

#else

  #if defined(ANDROID)
    #define __RADANDROID__ 1
    #define __RADDETECTED__ __RADANDROID__
  #endif

  #if defined(__QNX__)
    #define __RADQNX__ 2
    #define __RADDETECTED__ __RADQNX__
  #endif

  #if defined(_Windows) || defined(WIN32) || defined(__WINDOWS__) || defined(_WIN32) || defined(_WIN64) || defined(WINAPI_FAMILY)

    #ifdef WINAPI_FAMILY
      // If this is #defined, we might be in a Windows Store App. But
      // VC++ by default #defines this to a symbolic name, not an integer
      // value, and those names are defined in "winapifamily.h". So if
      // WINAPI_FAMILY is #defined, #include the header so we can parse it.
      #include <winapifamily.h>
      #define RAD_WINAPI_IS_APP (!WINAPI_FAMILY_PARTITION(WINAPI_PARTITION_DESKTOP))
      #if RAD_WINAPI_IS_APP
        #define __RADWINRTAPI__
        #define __RADWINRT__ 13
        #define __RADDETECTED__ __RADWINRT__
      #endif
    #else
      #define RAD_WINAPI_IS_APP 0
    #endif

    #ifndef __RADWINRT__
      // if we aren't WinRT, then we are plain old NT
      #define __RADNT__ 14
      #define __RADDETECTED__ __RADNT__
    #endif
  #endif

  #if defined(__APPLE__)
    #include "TargetConditionals.h"
    #if defined(TARGET_IPHONE_SIMULATOR) && TARGET_IPHONE_SIMULATOR
      #define __RADIPHONE__ 15
      #define __RADIPHONESIM__ 16
      #define __RADDETECTED__ __RADIPHONESIM__
    #elif defined(TARGET_OS_IPHONE) && TARGET_OS_IPHONE
      #define __RADIPHONE__ 15
      #define __RADDETECTED__ __RADIPHONE__
    #else
      #define __RADMAC__ 17
      #define __RADDETECTED__ __RADMAC__
    #endif
    // IOS/TVOIS/WATCHOS are subsets of __RADIPHONE__
    #if defined(TARGET_OS_IOS) && TARGET_OS_IOS
      #define __RADIOS__ 18
    #endif
    #if defined(TARGET_OS_TVOS) && TARGET_OS_TVOS
      #define __RADTVOS__ 19
    #endif
    #if defined(TARGET_OS_WATCHOS) && TARGET_OS_WATCHOS
      #define __RADWATCHOS__ 20
    #endif
  #endif

  #if defined(__EMSCRIPTEN__)
    #define __RADEMSCRIPTEN__  22
    #define __RADDETECTED__ __RADEMSCRIPTEN__
  #endif

  // linux is catch all for platforms when not compiling internally at rad
  #if defined(__linux__) && !defined(ANDROID)
    #define __RADLINUX__ 3
    #define __RADDETECTED__ __RADLINUX__
  #endif

  #if !defined( __RADDETECTED__) && !defined( USING_EGT )
    // this is the catch all, when you aren't building internally at rad
    #define __RADUNKNOWN__ 99
    #define __RADDETECTED__ __RADUNKNOWN__
  #endif

#endif


#if !defined(__RADDETECTED__)
  #error "egttypes.h did not detect your platform."
#endif

// ========================================================
// Now detect some architexture stuff

#define __RAD32__ // we have no non-at-least-32-bit cpus any more

#if defined(__arm__) || defined( _M_ARM )
  #define __RADARM__ 1
  #define __RADDETECTEDPROC__ __RADARM__
  #define __RADLITTLEENDIAN__
  #if defined(__ARM_NEON__) || defined(__ARM_NEON)
    #define __RADNEON__
  #endif
  #if !defined(__RADSOFTFP__) && !defined( __ARM_PCS_VFP )
    #define __RADSOFTFP__
  #endif
#endif
#if defined(__i386) || defined( __i386__ ) || defined( _M_IX86 ) || defined( _X86_ )
  #define __RADX86__ 2
  #if !defined __RADIPHONESIM__
    // only use mmx on PC, Win, Linux - not iphone sim!
    #define __RADMMX__
  #endif
  #define __RADDETECTEDPROC__ __RADX86__
  #define __RADLITTLEENDIAN__
#endif
#if defined(_x86_64) || defined( __x86_64__ ) || defined( _M_X64 ) || defined( _M_AMD64 )
  #define __RADX86__ 2
  #define __RADX64__ 3
  #if !defined __RADIPHONESIM__
    #define __RADMMX__
  #endif
  #define __RADDETECTEDPROC__ __RADX64__
  #define __RADLITTLEENDIAN__
#endif
#if defined(__powerpc) || defined( _M_PPC )
  #define __RADPPC__ 4
  #define __RADALTIVEC__
  #define __RADDETECTEDPROC__ __RADPPC__
  #define __RADBIGENDIAN__
#endif
#if defined( __aarch64__ ) || defined( __arm64__ ) || defined(_M_ARM64)
  #define __RADARM__ 1
  #define __RADARM64__ 6
  #define __RADDETECTEDPROC__ __RADARM64__
  #define __RADLITTLEENDIAN__
  #define __RADNEON__
#endif
#if defined(__EMSCRIPTEN__)
  #define __RADDETECTEDPROC__
  #define __RADLITTLEENDIAN__
#endif

#if !defined(__RADDETECTEDPROC__)
  #error "egttypes.h did not detect your processor type."
#endif

#if defined(__ppc64__) || defined(__aarch64__) || defined(_M_X64) || defined(__x86_64__) || defined(__x86_64) || defined(_M_ARM64)
  #define __RAD64__
  #define __RAD64REGS__  // need to set this for platforms that aren't 64-bit, but have 64-bit regs (old consoles)
#endif


// ========================================================
// handle exported function declarations:
//   in anything with __RADNOEXPORTS__, RADEXPFUNC == nothing (turn off exports with this flag)
//   in DLL, RADEXPFUNC == DLL export
//   in EXE, RADEXPFUNC == DLL import
//   in EXE with RADNOEXEEXPORTS, RADEXPFUNC == nothing (turn off imports in EXE with this flag)
//   in static lib, RADEXPFUNC == nothing
#if ( defined(__RADINSTATICLIB__) || defined(__RADNOEXPORTS__ ) || ( defined(__RADNOEXEEXPORTS__) && ( !defined(__RADINDLL__) ) && ( !defined(__RADINSTATICLIB__) ) ) )
  // if we are in a static lib, or exports are off, or if we are in an EXE we asked for no exe exports (or imports)
  //   then EXPFUNC is just a normal function
  #define RADEXPFUNC RADDEFFUNC
#else
  // otherwise, we use import or export base on the build flag __RADINDLL__
  #if defined(__RADINDLL__) 
    #define RADEXPFUNC RADDEFFUNC RADDLLEXPORTDLL
  #else
    #define RADEXPFUNC RADDEFFUNC RADDLLIMPORTDLL
  #endif
#endif

#if defined(__RADANDROID__)
  #define RADRESTRICT __restrict
  #define RADSTRUCT struct __attribute__((__packed__))

  #define RADLINK
  #define RADEXPLINK
  #define RADDLLEXPORTDLL __attribute__((visibility("default")))
  #define RADDLLIMPORTDLL
#endif

#if defined(__RADQNX__)
  #define RADRESTRICT __restrict
  #define RADSTRUCT struct __attribute__((__packed__))

  #define RADLINK
  #define RADEXPLINK RADLINK
  #define RADDLLEXPORTDLL
  #define RADDLLIMPORTDLL
#endif

#if defined(__RADLINUX__) || defined(__RADUNKNOWN__)
  #define RADRESTRICT __restrict
  #define RADSTRUCT struct __attribute__((__packed__))

  #if defined(__RADX86__) && !defined(__RADX64__)
    #define RADLINK __attribute__((cdecl))
    #define RADEXPLINK __attribute__((cdecl))
  #else
    #define RADLINK
    #define RADEXPLINK
  #endif
  // for linux, we assume you are building with hidden visibility,
  //   so for RADEXPFUNC, we turn the vis back on...
  #define RADDLLEXPORTDLL __attribute__((visibility("default")))
  #define RADDLLIMPORTDLL
#endif

#if defined(__RADNT__)
  #define __RADWIN__
  #if _MSC_VER >= 1400
    #define RADRESTRICT __restrict
  #else
    // vc6 and older
    #define RADRESTRICT
    #define __RADNOVARARGMACROS__
  #endif
  #define RADSTRUCT struct 

  #define RADLINK __stdcall
  #define RADEXPLINK __stdcall

  #define RADDLLEXPORTDLL __declspec(dllexport)
  #ifdef __RADX32__
    // on weird NT DLLs built to run on Linux and Mac, no imports
    #define RADDLLIMPORTDLL
  #else
    // normal win32 dll import
    #define RADDLLIMPORTDLL __declspec(dllimport)
  #endif
#endif

#if defined(__RADWINRT__)
  #define __RADWIN__
  #if defined(__RADARM__) // no non-NEON ARMs in WinRT devices (so far)
    #define __RADNEON__
  #endif
  #define RADRESTRICT __restrict
  #define RADSTRUCT struct 

  #define RADLINK __stdcall
  #define RADEXPLINK __stdcall
  #define RADDLLEXPORTDLL __declspec(dllexport)
  #define RADDLLIMPORTDLL __declspec(dllimport)
#endif

#if defined(__RADIPHONE__)
  #define __RADMACAPI__
  #define RADRESTRICT __restrict
  #define RADSTRUCT struct __attribute__((__packed__))

  #define RADLINK
  #define RADEXPLINK
  #define RADDLLEXPORTDLL
  #define RADDLLIMPORTDLL
#endif

#if defined(__RADMAC__)
  #define __RADMACH__
  #define __RADMACAPI__
  #define RADRESTRICT __restrict
  #define RADSTRUCT struct __attribute__((__packed__))

  #define RADLINK
  #define RADEXPLINK
  // for mac, we assume you are building with hidden visibility,
  //   so for RADEXPFUNC, we turn the vis back on...
  #define RADDLLEXPORTDLL __attribute__((visibility("default")))
  #define RADDLLIMPORTDLL

  #ifdef TARGET_API_MAC_CARBON
    #if TARGET_API_MAC_CARBON
      #ifndef __RADCARBON__
        #define __RADCARBON__
      #endif
    #endif
  #endif
#endif

#if defined(__RADEMSCRIPTEN__)
  #include <emscripten.h>
  #define RADRESTRICT __restrict
  #define RADSTRUCT struct __attribute__((__packed__))

  #define RADLINK
  #define RADEXPLINK EMSCRIPTEN_KEEPALIVE
  #define RADDLLEXPORTDLL 
  #define RADDLLIMPORTDLL
#endif

#ifndef RADLINK
  #error RADLINK was not defined.
#endif

#ifdef _MSC_VER
  #define RADINLINE __inline
#else
  #define RADINLINE inline
#endif

//===========================================================================
// RR_NUMBERNAME is a macro to make a name unique, so that you can use it to declare
//    variable names and they won't conflict with each other
// using __LINE__ is broken in MSVC with /ZI , but __COUNTER__ is an MSVC extension that works

#ifdef _MSC_VER
  #define RR_NUMBERNAME(name) RR_STRING_JOIN(name,__COUNTER__)
#else
  #define RR_NUMBERNAME(name) RR_STRING_JOIN(name,__LINE__)
#endif

//===================================================================
// simple compiler assert
// this happens at declaration time, so if it's inside a function in a C file, drop {} around it
#ifndef RR_COMPILER_ASSERT
  #if defined(__clang__)
    #define RR_COMPILER_ASSERT_UNUSED __attribute__((unused))  // hides warnings when compiler_asserts are in a local scope
  #else
    #define RR_COMPILER_ASSERT_UNUSED
  #endif

  #define RR_COMPILER_ASSERT(exp)   typedef char RR_NUMBERNAME(_dummy_array) [ (exp) ? 1 : -1 ] RR_COMPILER_ASSERT_UNUSED
#endif


//===========================================
// first, we set defines for each of the types

#define RAD_S8 signed char
#define RAD_U8 unsigned char
#define RAD_U16 unsigned short
#define RAD_S16 signed short

#if defined(__RAD64__) 
  #define RAD_U32 unsigned int
  #define RAD_S32 signed int

  // pointers are 64 bits.
  #if ( defined(_MSC_VER) && _MSC_VER >= 1300 && defined(_Wp64) && _Wp64 )
    #define RAD_SINTa __w64 signed __int64
    #define RAD_UINTa __w64 unsigned __int64
  #else 
    // non-vc.net compiler or /Wp64 turned off
    #define RAD_UINTa unsigned long long
    #define RAD_SINTa signed long long
  #endif
#endif

#if defined(__RAD32__) && !defined(__RAD64__)
  #define RAD_U32 unsigned int
  #define RAD_S32 signed int

  #if ( ( defined(_MSC_VER) && (_MSC_VER >= 1300 ) ) && ( defined(_Wp64) && ( _Wp64 ) ) )
    #define RAD_SINTa __w64 signed long
    #define RAD_UINTa __w64 unsigned long
  #else 
    // non-vc.net compiler or /Wp64 turned off
    #ifdef _Wp64
      #define RAD_SINTa signed long
      #define RAD_UINTa unsigned long
    #else
      #define RAD_SINTa signed int
      #define RAD_UINTa unsigned int
    #endif
  #endif
#endif

#define RAD_F32 float
#define RAD_F64 double

#if defined(_MSC_VER)
  #define RAD_U64 unsigned __int64
  #define RAD_S64 signed __int64
#else
  #define RAD_U64 unsigned long long
  #define RAD_S64 signed long long
#endif


//================================================================
// Then, we either typedef or define them based on switch settings

#if !defined(RADNOTYPEDEFS)  // this define will turn off typedefs

  #ifndef S8_DEFINED
  #define S8_DEFINED
  typedef RAD_S8 S8;
  #endif

  #ifndef U8_DEFINED
  #define U8_DEFINED
  typedef RAD_U8 U8;
  #endif

  #ifndef S16_DEFINED
  #define S16_DEFINED
  typedef RAD_S16 S16;
  #endif

  #ifndef U16_DEFINED
  #define U16_DEFINED
  typedef RAD_U16 U16;
  #endif

  #ifndef S32_DEFINED
  #define S32_DEFINED
  typedef RAD_S32 S32;
  #endif

  #ifndef U32_DEFINED
  #define U32_DEFINED
  typedef RAD_U32 U32;
  #endif

  #ifndef S64_DEFINED
  #define S64_DEFINED
  typedef RAD_S64 S64;
  #endif

  #ifndef U64_DEFINED
  #define U64_DEFINED
  typedef RAD_U64 U64;
  #endif

  #ifndef F32_DEFINED
  #define F32_DEFINED
  typedef RAD_F32 F32;
  #endif

  #ifndef F64_DEFINED
  #define F64_DEFINED
  typedef RAD_F64 F64;
  #endif

  #ifndef SINTa_DEFINED
  #define SINTa_DEFINED
  typedef RAD_SINTa SINTa;
  #endif

  #ifndef UINTa_DEFINED
  #define UINTa_DEFINED
  typedef RAD_UINTa UINTa;
  #endif

  #ifndef RRBOOL_DEFINED
    #define RRBOOL_DEFINED
    typedef S32 rrbool;
    typedef S32 RRBOOL;
  #endif

#elif !defined(RADNOTYPEDEFINES)  // this define will turn off type defines

  #ifndef S8_DEFINED
  #define S8_DEFINED
  #define S8 RAD_S8
  #endif

  #ifndef U8_DEFINED
  #define U8_DEFINED
  #define U8 RAD_U8
  #endif

  #ifndef S16_DEFINED
  #define S16_DEFINED
  #define S16 RAD_S16
  #endif

  #ifndef U16_DEFINED
  #define U16_DEFINED
  #define U16 RAD_U16
  #endif

  #ifndef S32_DEFINED
  #define S32_DEFINED
  #define S32 RAD_S32
  #endif

  #ifndef U32_DEFINED
  #define U32_DEFINED
  #define U32 RAD_U32
  #endif

  #ifndef S64_DEFINED
  #define S64_DEFINED
  #define S64 RAD_S64
  #endif

  #ifndef U64_DEFINED
  #define U64_DEFINED
  #define U64 RAD_U64
  #endif

  #ifndef F32_DEFINED
  #define F32_DEFINED
  #define F32 RAD_F32
  #endif

  #ifndef F64_DEFINED
  #define F64_DEFINED
  #define F64 RAD_F64
  #endif

  #ifndef SINTa_DEFINED
  #define SINTa_DEFINED
  #define SINTa RAD_SINTa
  #endif

  #ifndef UINTa_DEFINED
  #define UINTa_DEFINED
  #define UINTa RAD_UINTa
  #endif

  #ifndef RRBOOL_DEFINED
    #define RRBOOL_DEFINED
    #define rrbool S32
    #define RRBOOL S32
  #endif

#endif

#endif // __RADRES__

#endif // __EGTTYPESH__
