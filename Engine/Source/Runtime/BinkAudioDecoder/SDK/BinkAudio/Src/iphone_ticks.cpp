// Copyright Epic Games, Inc. All Rights Reserved.
#include "ticks.h"

#include <mach/mach_time.h>
#include <unistd.h>

U64 baue_ticks( void )
{
  return mach_absolute_time();
}
