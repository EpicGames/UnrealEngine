// Copyright Epic Games, Inc. All Rights Reserved.
#include "ticks.h"
#include <unistd.h>

#ifdef __RADARM__

  static inline unsigned long long rdtsc()
  {
    unsigned long long tsc;
    asm volatile("mrs %0, cntvct_el0" : "=r" (tsc));
    return tsc;
  }

  #define get_rdtsc(x) x = rdtsc();

#elif defined(__RADX86__)

#define get_rdtsc(x) __asm__ volatile ("rdtsc" : "=A" (x))

#else

#error no 32 bit

#endif

U64 baue_ticks( void )
{
  U64 ticks;
  get_rdtsc( ticks );
  return ticks;
}
