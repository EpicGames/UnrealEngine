// Copyright Epic Games, Inc. All Rights Reserved.
#ifndef __RADRR_COREH__
  #include "rrCore.h"
#endif

#ifdef __RADMAC__
  #include <memory.h>
#endif


#include "popmal.h"

typedef struct PM
{
  void * * * ptrs;
  void * * * pushptr;
  U64 * amt;
  U64 * pushamt;
  U64 pushtot;
  U32 pushcur;
  U32 cursize;
} PM;


RADDEFFUNC void RADLINK pushmallocinit(void* base,U32 num_ptrs)
{
  PM * p = (PM *)base;
  typedef char blah[ ( PushMallocBytesForXPtrs( 0 ) >= sizeof(PM) ) ? 1 : -1 ];

  p->ptrs = (void * * *) ( ( (char*) base ) + PushMallocBytesForXPtrs( 0 ) );
  p->amt = (U64 *) ( ( (char*) p->ptrs ) + ( num_ptrs * sizeof(void*) ) );
  p->pushtot = 0;
  p->pushcur = 0;
  p->pushptr = p->ptrs;
  p->pushamt = p->amt;
  p->cursize = num_ptrs;
}

RADDEFFUNC void RADLINK pushmalloc(void* base, void * ptr,U64 amt)
{
  PM * p = (PM *)base;
  if ( p->cursize == p->pushcur )
  {
    RR_BREAK();
  }

  {
    U64 last,next;

    amt=Round32(amt);
    last=((p->pushtot/32)&31)+1;
    next=(amt/32)&31;

    // make sure the up 32 separate mallocs use distinct sets
    amt+=(((32+last-next)&31)*32);
  }

  p->pushtot+=amt;
  p->pushamt[p->pushcur]=amt;
  p->pushptr[p->pushcur++]=(void * *)ptr;
}

RADDEFFUNC void RADLINK pushmalloco(void* base, void * ptr,U64 amt)
{
  PM * p = (PM *)base;
  pushmalloc(base, ptr, amt);
  p->pushamt[p->pushcur-1]|=1;// mark it as special
}

RADDEFFUNC U64 RADLINK popmalloctotal( void* base )
{
  PM * p = (PM *)base;
  if ( p == 0 ) return 0;
  return( p->pushtot );
}

RADDEFFUNC void * RADLINK popmalloc(void* base, U64 amt, void* (*allocator)(UINTa bytes))
{
  PM * p = (PM *)base;
  void * ptr;

  amt=Round32(amt);

  if ( p == 0 )
  {
    return allocator( (UINTa)amt );
  }

  ptr=allocator((UINTa)(p->pushtot+amt));
  
  p->pushtot=0;
  if (ptr) {
    U32 i;
    U8 * np;

    np=((U8 *)ptr)+amt;
    for(i=0;i<p->pushcur;i++) 
    {
      if ( p->pushamt[i]&1 )
        (*(void**)(((U8*)ptr)+((UINTa)p->pushptr[i])))=np;
      else
        (*(p->pushptr[i]))=np;
      np=np+(p->pushamt[i]&~1);
    }


  }
  p->pushcur=0;
  return(ptr);
}
