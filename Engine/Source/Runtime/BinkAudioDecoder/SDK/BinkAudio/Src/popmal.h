// Copyright Epic Games, Inc. All Rights Reserved.
#ifndef __POPMALH__
#define __POPMALH__

#ifndef __RADRR_COREH__
  #include "rrCore.h"
#endif

#define Round32( num ) ( ( ( num ) + 31 ) & ~31 )

#define PushMallocBytesForXPtrs( X ) ( ( X * sizeof(void*) ) + ( X * sizeof(U64) ) + 64 )


#ifdef WRAP_PUBLICS
#define rfmerge3(name,add) name##add
#define rfmerge2(name,add) rfmerge3(name,add)
#define rfmerge(name)      rfmerge2(name,WRAP_PUBLICS)
#define pushmallocinit                    rfmerge(pushmallocinit)
#define pushmalloc                        rfmerge(pushmalloc)
#define pushmalloco                       rfmerge(pushmalloco)
#define popmalloctotal                    rfmerge(popmalloctotal)
#define popmalloc                         rfmerge(popmalloc)
#endif


RADDEFFUNC void RADLINK pushmallocinit(void * base,U32 num_ptrs);
RADDEFFUNC void RADLINK pushmalloc( void * base, void  * ptr, U64 amt );

// ptr is an offset within the final allocation that will be popped (for allocating ptrs within a structure that you allocate with popmalloc)
RADDEFFUNC void RADLINK pushmalloco(void* base, void * ptr,U64 amt);

RADDEFFUNC U64 RADLINK popmalloctotal( void * base );

RADDEFFUNC void  * RADLINK popmalloc( void * base, U64 amt, void* (*allocator)(UINTa bytes));

#define popfree(ptr, memfree) memfree(ptr)

#endif //  __POPMALH__
