// Copyright Epic Games, Inc. All Rights Reserved.
#include "ticks.h"
#include <sys/time.h>
#include <unistd.h>

U64 baue_ticks( void )
{
  struct timeval tv;
  gettimeofday(&tv, NULL);
  return ( (U64)(U32)tv.tv_sec * 1000000L ) + (U64)(U32)tv.tv_usec;
}