// Copyright Epic Games, Inc. All Rights Reserved.
#include "rrCore.h"
#include "radfft.h"

// Algorithm:
//
// This code computes FFTs and derived transforms using a decimation-in-time, conjugate-pair
// split-radix algorithm that closely follows Blake et al., "The Fastest Fourier Transform
// in the South", albeit without any of the runtime code generation.
//
// FFTS considers only 64-bit x86 and ARM, and thus assumes that 16 vector registers are
// available, and that the cache is at least 8-way set associative. This implementation also
// tries to perform well on 32-bit x86; this reduces the practical size of the recursion
// base cases from 8 to 4. As a side effect, we also don't need more than 4 cache ways, which
// is the right choice should we want to port this to order in-order cores.
//
// We also don't bother with the transition codelets in the base cases. The base case loops
// process as much data as possible while staying aligned; we don't bother with specialized
// SIMD code for the transition to another loop size though, we just run scalar versions
// at the edges. This is slightly less efficient, but a lot simpler.
//
// Real FFTs are computed using a complex FFT of half the size plus a Cooley-Tukey radix-2
// DIT step (the standard "packing" algorithm).
//
// The DCTs are computed using the standard reduction of N-element DCT-II and DCT-III to
// N-element real FFTs.
//
// Some more notes:
// - I looked into DCT-IIs merging their modulate step with the preceding rfft post-process.
//   This works and saves a pass over the data, but the code is more complicated and wasn't
//   faster in my tests. (Same goes for merging DCT-III modulate with rifft pre-process)

//#define TABLEGEN // Compile this file with /DTABLEGEN to write out prepared tables!

#ifndef FORCE_NO_FFT_TABLES
#define USETABLES // Use pre-generated twiddle/permutation tables
#endif

#ifdef TABLEGEN // TABLEGEN shouldn't use the prebuilt tables or we're getting circular! :)
#undef USETABLES
#endif

#define FFTASSERT(cond) if (!(cond)) RR_BREAK()
#define FFTALIGNED(type, name) static RAD_ALIGN(type const, name, RADFFT_ALIGN)

#define ALIGNHINT(var,align)
#define FFTTABLE(type, name) FFTALIGNED(type, name)

#if defined __has_builtin
#define RAD_HAS_BUILTIN(n) __has_builtin(n)
#else
#define RAD_HAS_BUILTIN(n) 0
#endif

#if RAD_HAS_BUILTIN(__builtin_cos) && RAD_HAS_BUILTIN(__builtin_sin)
#define cos(v) (__builtin_cos(v))
#define sin(v) (__builtin_sin(v))
#define _LIBCPP_MATH_H
#else
#include <math.h>
#endif

#ifdef BIG_OLE_FFT
static UINTa const kMaxN = 4096;   // Largest FFT size we support. This is easy to change.
#else
static UINTa const kMaxN = 2048;   // Largest FFT size we support. This is easy to change.
#endif
static UINTa const kLeafN = 4;     // Size of leaf transforms. This isn't easy to change at all. :)
static UINTa const kMaxPlan = 256; // Largest FFT size we have pre-planned.

typedef U16 Index; // Index into FFT - need to change this if kMaxN > 65536

// For the small (leaf) FFTs, instead of doing an explicit recursion, we just loop over this list
// (the "plan"), which is a sequence of conjugate split-radix steps to do: offset and number of
// loop iterations.
//
// We have a decimation-in-time decomposition, which builds up from smaller towards larger FFTs.
// That means one plan is sufficient for all FFT sizes up to kMaxPlan: stopping after the first
// step with Nloop >= N/4 yields a N-element FFT.
struct PlanElement
{
    Index offs;
    Index Nloop;
};

// Prepared recursion plan up to kMaxPlan. This stores the sizes and offsets
// of the FFT passes. Generated by this program:
//
// ----
// #include <stdio.h>
// void plan(int offs, int N, int Nleaf)
// {
//     static int counter = 0;
//     if (N <= Nleaf)
//         return;
//     // Split-radix recursion pattern
//     plan(offs, N/2, Nleaf);
//     plan(offs + N/2, N/4, Nleaf);
//     plan(offs + 3*N/4, N/4, Nleaf);
//     printf("{ %3d, %2d },", offs, N/4);
//     printf((++counter % 8) ? " " : "\n");
// }
// int main()
// {
//     plan(0, 256, 4); // kMaxPlan=256, kLeafN=4
//     printf("\n");
//     return 0;
// }

RADDEFSTART

static
PlanElement s_recursion_plan[] =
{
    {   0,  2 }, {   0,  4 }, {  16,  2 }, {  24,  2 }, {   0,  8 }, {  32,  2 }, {  32,  4 }, {  48,  2 },
    {  48,  4 }, {   0, 16 }, {  64,  2 }, {  64,  4 }, {  80,  2 }, {  88,  2 }, {  64,  8 }, {  96,  2 },
    {  96,  4 }, { 112,  2 }, { 120,  2 }, {  96,  8 }, {   0, 32 }, { 128,  2 }, { 128,  4 }, { 144,  2 },
    { 152,  2 }, { 128,  8 }, { 160,  2 }, { 160,  4 }, { 176,  2 }, { 176,  4 }, { 128, 16 }, { 192,  2 },
    { 192,  4 }, { 208,  2 }, { 216,  2 }, { 192,  8 }, { 224,  2 }, { 224,  4 }, { 240,  2 }, { 240,  4 },
    { 192, 16 }, {   0, 64 },
};

// Twiddles and permutation tables for the FFT.
// If you change this, update the TABLEGEN code below!
#ifdef USETABLES
#ifdef BIG_OLE_FFT
#include "radfft_tables_4096.inl"
#else
#include "radfft_tables.inl"
#endif
#else
FFTALIGNED(rfft_complex, s_twiddles[(kMaxN / 4) * 2]);   // Regular FFT twiddles: N-elem FFT needs a quarter circle; *2 because we store all "mip levels"
FFTALIGNED(rfft_complex, s_dct_twiddles[kMaxN / 4 + kMaxN / 2]); // DCT needs one eight of a circle at 4N rate, so we need two more.
Index s_permute[(kMaxN / kLeafN) * 2]; // *2 because we keep a "mip chain" for smaller transforms.
#endif

RADDEFEND

// --------------------------------------------------------------------------
// Kernel types
// --------------------------------------------------------------------------

// Real (I)FFT pre/post-pass
typedef void RFFTPrePostKernel(rfft_complex data[], UINTa kEnd, UINTa N4);
// Complex (I)FFT
typedef void CFFTKernel(rfft_complex data[], PlanElement const *plan, UINTa Nover4);
// Radix4 or 2xRadix2 CFFT base cases
typedef void BaseKernel(rfft_complex out[], rfft_complex const in0[], rfft_complex const in1[], rfft_complex const in2[], rfft_complex const in3[], UINTa j0, UINTa j1, Index const *perm);
// (I)DCT merge/split
typedef void MergeSplitKernel(F32 out[], F32 const in[], UINTa N);
typedef void MergeSplitKernelS16(S16 out[], F32 scale, F32 const in[], UINTa N);
typedef void MergeSplitKernelS16S(S16 out[], S16 left[], F32 scale, F32 const in[], UINTa N);
// (I)DCT modulate
typedef void ModulateKernel(F32 out[], F32 const in[], UINTa N, UINTa Nlast, rfft_complex const *twiddle);

struct KernelSet
{
    RFFTPrePostKernel    *rfpost;      // Real FFT post
    RFFTPrePostKernel    *ripre;       // Real IFFT pre
    CFFTKernel           *cfpass;      // Complex FFT pass
    CFFTKernel           *cipass;      // Complex IFFT pass
    BaseKernel           *radix4;      // Radix4 base case 
    BaseKernel           *radix2_2;    // 2x Radix2 base case
    MergeSplitKernel     *dct_split;   // DCT-II split pass
    MergeSplitKernel     *dct_merge;   // DCT-III merge pass
    MergeSplitKernelS16  *dct_merge_s16;   // DCT-III merge pass with s16 out
    MergeSplitKernelS16S *dct_merge_s16s;   // DCT-III merge pass with s16 stereo out (left chan passed in)
    ModulateKernel       *dct2_mod;    // DCT-II modulation pass
    ModulateKernel       *dct3_mod;    // DCT-III modulation pass
};

// --------------------------------------------------------------------------
// Scalar kernels. These are always there.
// --------------------------------------------------------------------------

// Size 1 or 2 complex (I)FFTs - trivial (for base cases)
static void scalar_cfft_tiny(rfft_complex out[], rfft_complex const in[], UINTa N)
{
    if (N == 2)
    {
        rfft_complex a = in[0];
        rfft_complex b = in[1];
        out[0].re = a.re + b.re;
        out[0].im = a.im + b.im;
        out[1].re = a.re - b.re;
        out[1].im = a.im - b.im;
    }
    else if (N == 1)
        out[0] = in[0];
}

// Real FFT post-pass.
static void scalar_rfpost(rfft_complex out[], UINTa kEnd, UINTa N4)
{
    rfft_complex const *twiddle = s_twiddles + N4;
    rfft_complex *out0 = out + 1;
    rfft_complex *out1 = out + (N4 * 2) - 1;

    // out'[k] = 0.5 * ((1 - i*conj(twiddle[k])) * out[k] + (1 + i*conj(twiddle[k])) * conj(out[N-k]))
    for (UINTa k = 1; k < kEnd; ++k, ++out0, --out1)
    {
        rfft_complex const &w = twiddle[k];
        F32 dr = 0.5f * (out1->re - out0->re);
        F32 di = 0.5f * (out0->im + out1->im);
        F32 evr = out0->re + dr;
        F32 evi = out0->im - di;
        F32 odr = w.re*di + w.im*dr;
        F32 odi = w.re*dr - w.im*di;

        out0->re = evr + odr;
        out0->im = evi + odi;
        out1->re = evr - odr;
        out1->im = odi - evi;
    }
}

// Real IFFT pre-pass.
static void scalar_ripre(rfft_complex out[], UINTa kEnd, UINTa N4)
{
    rfft_complex const *twiddle = s_twiddles + N4;
    rfft_complex *out0 = out + 1;
    rfft_complex *out1 = out + (N4 * 2) - 1;

    for (UINTa k = 1; k < kEnd; ++k, ++out0, --out1)
    {
        rfft_complex const &w = twiddle[k];
        F32 dr = 0.5f * (out0->re - out1->re);
        F32 di = 0.5f * (out0->im + out1->im);
        F32 evr = out0->re - dr;
        F32 evi = out0->im - di;
        F32 odr = w.re*di + w.im*dr;
        F32 odi = w.re*dr - w.im*di;

        out0->re = evr - odr;
        out0->im = evi + odi;
        out1->re = evr + odr;
        out1->im = odi - evi;
    }
}

// Scalar conjugate split-radix forward pass.
static void scalar_cfpass(rfft_complex data[], PlanElement const *plan, UINTa Nover4)
{
    --plan;

    do
    {
        ++plan;
        UINTa N1 = plan->Nloop;
        rfft_complex *out = data + plan->offs;
        rfft_complex const *twiddle = s_twiddles + N1;

        // k=0 has twiddle factors 1 so we can save a bunch of work
        // (this is worthwhile because we keep subdividing into shorter and
        // shorter transforms; it's not just a one-time thing, we win on
        // every level of the recursion)
        {
            rfft_complex &x0 = out[0*N1];
            rfft_complex &x1 = out[1*N1];
            rfft_complex &x2 = out[2*N1];
            rfft_complex &x3 = out[3*N1];

            F32 Zsumr = x2.re + x3.re;
            F32 Zsumi = x2.im + x3.im;
            F32 Zdifr = x2.im - x3.im;
            F32 Zdifi = x3.re - x2.re;

            F32 U0r = x0.re;
            F32 U0i = x0.im;
            x2.re = U0r - Zsumr;
            x2.im = U0i - Zsumi;
            x0.re = U0r + Zsumr;
            x0.im = U0i + Zsumi;

            F32 U1r = x1.re;
            F32 U1i = x1.im;
            x3.re = U1r - Zdifr;
            x3.im = U1i - Zdifi;
            x1.re = U1r + Zdifr;
            x1.im = U1i + Zdifi;
            ++out;
        }

        for (UINTa k = 1; k < N1; ++k)
        {
            rfft_complex const &w = twiddle[k];
            rfft_complex &x0 = out[0*N1];
            rfft_complex &x1 = out[1*N1];
            rfft_complex &x2 = out[2*N1];
            rfft_complex &x3 = out[3*N1];

            // This is the general case: (complex values, r=real part, i=imaginary part)
            // w_k is the twiddle factor, (omega)^k.
            //
            //   Z_k    =      w_k  * x[k + 2N/4]
            //   Z'_k   = conj(w_k) * x[k + 3N/4]
            //   U0     =             x[k + 0N/4]
            //   U1     =             x[k + 1N/4]
            //
            //   Zsum_k = Z_k + Z'_k
            //   Zdif_k = -i * (Z_k - Z'_k)
            //
            //   new_x[k + 0N/4] = U0 + Zsum_k = U0 +   (Z_k + Z'_k)
            //   new_x[k + 1N/4] = U1 + Zdif_k = U1 - i*(Z_k - Z'_k)
            //   new_x[k + 2N/4] = U0 - Zsum_k = U0 -   (Z_k + Z'_k)
            //   new_x[k + 3N/4] = U1 - Zdif_k = U1 + i*(Z_k - Z'_k)

            F32 Zkr  = w.re*x2.re - w.im*x2.im;  
            F32 Zki  = w.re*x2.im + w.im*x2.re;
            F32 Zpkr = w.re*x3.re + w.im*x3.im;
            F32 Zpki = w.re*x3.im - w.im*x3.re;
            
            F32 Zsumr = Zkr + Zpkr;
            F32 Zsumi = Zki + Zpki;
            F32 Zdifr = Zki - Zpki;
            F32 Zdifi = Zpkr - Zkr;

            F32 U0r = x0.re;
            F32 U0i = x0.im;
            x2.re = U0r - Zsumr;
            x2.im = U0i - Zsumi;
            x0.re = U0r + Zsumr;
            x0.im = U0i + Zsumi;
            F32 U1r = x1.re;
            F32 U1i = x1.im;
            x3.re = U1r - Zdifr;
            x3.im = U1i - Zdifi;
            x1.re = U1r + Zdifr;
            x1.im = U1i + Zdifi;
            ++out;
        }
    } while (plan->Nloop < Nover4);
}

// Scalar conjugate split-radix inverse pass.
static void scalar_cipass(rfft_complex data[], PlanElement const *plan, UINTa Nover4)
{
    --plan;

    do
    {
        ++plan;
        UINTa N1 = plan->Nloop;
        rfft_complex *out = data + plan->offs;
        rfft_complex const *twiddle = s_twiddles + N1;

        // k=0 has twiddle factors 1 so we can save a bunch of work
        // (this is worthwhile because we keep subdividing into shorter and
        // shorter transforms; it's not just a one-time thing, we win on
        // every level of the recursion)
        {
            rfft_complex &x0 = out[0*N1];
            rfft_complex &x1 = out[1*N1];
            rfft_complex &x2 = out[2*N1];
            rfft_complex &x3 = out[3*N1];

            F32 Zsumr = x2.re + x3.re;
            F32 Zsumi = x2.im + x3.im;
            F32 Zdifr = x3.im - x2.im;
            F32 Zdifi = x2.re - x3.re;

            F32 U0r = x0.re;
            F32 U0i = x0.im;
            x2.re = U0r - Zsumr;
            x2.im = U0i - Zsumi;
            x0.re = U0r + Zsumr;
            x0.im = U0i + Zsumi;
            F32 U1r = x1.re;
            F32 U1i = x1.im;
            x3.re = U1r - Zdifr;
            x3.im = U1i - Zdifi;
            x1.re = U1r + Zdifr;
            x1.im = U1i + Zdifi;
            ++out;
        }

        for (UINTa k = 1; k < N1; ++k)
        {
            rfft_complex const &w = twiddle[k];
            rfft_complex &x0 = out[0*N1];
            rfft_complex &x1 = out[1*N1];
            rfft_complex &x2 = out[2*N1];
            rfft_complex &x3 = out[3*N1];

            // This is the general case: (complex values, r=real part, i=imaginary part)
            // w_k is the twiddle factor, (omega)^k.
            //
            //   Z_k    = conj(w_k) * x[k + 2N/4]
            //   Z'_k   =      w_k  * x[k + 3N/4]
            //   U0     =             x[k + 0N/4]
            //   U1     =             x[k + 1N/4]
            //
            //   Zsum_k = Z_k + Z'_k
            //   Zdif_k = i * (Z_k - Z'_k)
            //
            //   new_x[k + 0N/4] = U0 + Zsum_k = U0 +   (Z_k + Z'_k)
            //   new_x[k + 1N/4] = U1 + Zdif_k = U1 + i*(Z_k - Z'_k)
            //   new_x[k + 2N/4] = U0 - Zsum_k = U0 -   (Z_k + Z'_k)
            //   new_x[k + 3N/4] = U1 - Zdif_k = U1 - i*(Z_k - Z'_k)
            //
            // Note that this is essentially the same as the forward transform (cfpass), except
            // the twiddle factors (the w_k and i) are conjugated.

            F32 Zkr  = w.re*x2.re + w.im*x2.im;  
            F32 Zki  = w.re*x2.im - w.im*x2.re;
            F32 Zpkr = w.re*x3.re - w.im*x3.im;
            F32 Zpki = w.re*x3.im + w.im*x3.re;
            
            F32 Zsumr = Zkr + Zpkr;
            F32 Zsumi = Zki + Zpki;
            F32 Zdifr = Zpki - Zki;
            F32 Zdifi = Zkr - Zpkr;

            F32 U0r = x0.re;
            F32 U0i = x0.im;
            x2.re = U0r - Zsumr;
            x2.im = U0i - Zsumi;
            x0.re = U0r + Zsumr;
            x0.im = U0i + Zsumi;
            F32 U1r = x1.re;
            F32 U1i = x1.im;
            x3.re = U1r - Zdifr;
            x3.im = U1i - Zdifi;
            x1.re = U1r + Zdifr;
            x1.im = U1i + Zdifi;
            ++out;
        }
    } while (plan->Nloop < Nover4);
}

// Radix-4 FFT codelet. To get the inverse (up to scale), swap in1 and in3.
static void scalar_radix4(rfft_complex out[], rfft_complex const in0[], rfft_complex const in1[], rfft_complex const in2[], rfft_complex const in3[], UINTa j0, UINTa j1, Index const *perm)
{
    for (UINTa j = j0; j < j1; ++j)
    {
        rfft_complex const &a = in0[j];
        rfft_complex const &b = in2[j];
        rfft_complex const &c = in1[j];
        rfft_complex const &d = in3[j];
        rfft_complex *o = out + perm[j];

        F32 Ar = a.re + b.re;
        F32 Ai = a.im + b.im;
        F32 Br = a.re - b.re;
        F32 Bi = a.im - b.im;
        F32 Cr = c.re + d.re;
        F32 Ci = c.im + d.im;
        F32 Dr = c.im - d.im;
        F32 Di = d.re - c.re;

        o[0].re = Ar + Cr;
        o[0].im = Ai + Ci;
        o[2].re = Ar - Cr;
        o[2].im = Ai - Ci;
        o[1].re = Br + Dr;
        o[1].im = Bi + Di;
        o[3].re = Br - Dr;
        o[3].im = Bi - Di;
    }
}

// 2x Radix-2 FFT codelet. Self-inverse (up to scale).
static void scalar_2radix2(rfft_complex out[], rfft_complex const in0[], rfft_complex const in1[], rfft_complex const in2[], rfft_complex const in3[], UINTa j0, UINTa j1, Index const *perm)
{
    for (UINTa j = j0; j < j1; ++j)
    {
        rfft_complex const &a = in0[j];
        rfft_complex const &b = in2[j];
        rfft_complex const &c = in3[j];
        rfft_complex const &d = in1[j];
        rfft_complex *o = out + perm[j];

        o[0].re = a.re + b.re;
        o[0].im = a.im + b.im;
        o[1].re = a.re - b.re;
        o[1].im = a.im - b.im;
        o[2].re = c.re + d.re;
        o[2].im = c.im + d.im;
        o[3].re = c.re - d.re;
        o[3].im = c.im - d.im;
    }
}

// DCT even/odd split
static void scalar_dct_split(F32 out[], F32 const in[], UINTa N)
{
    UINTa N2 = N / 2;
    F32 *out0 = out;
    F32 *out1 = out + N;
    F32 const *inp = in;

    // Even-indexed input elems go to first half of out
    // Odd-indexed input elems go to second half of out, in reverse order
    for (UINTa k = 0; k < N2; ++k)
    {
        F32 a = *inp++;
        F32 b = *inp++;
        *out0++ = a;
        *--out1 = b;
    }
}

// DCT merge (inverse of split)
static void scalar_dct_merge(F32 out[], F32 const in[], UINTa N)
{
    UINTa N2 = N / 2;
    F32 *outp = out;
    F32 const *in0 = in;
    F32 const *in1 = in + N;

    for (UINTa k = 0; k < N2; ++k)
    {
        F32 a = *in0++;
        F32 b = *--in1;
        *outp++ = a;
        *outp++ = b;
    }
}

// DCT merge (inverse of split)
static void scalar_dct_merge_s16(S16 out[], F32 scale, F32 const in[], UINTa N)
{
    UINTa N2 = N / 2;
    S16 *outp = out;
    F32 const *in0 = in;
    F32 const *in1 = in + N;

    for (UINTa k = 0; k < N2; ++k)
    {
        F32 a = *in0++;
        F32 b = *--in1;
        S32 va = (S32)(a*scale);
        S32 vb = (S32)(b*scale);
        if ( va > 32767 ) va = 32767; if ( va <= -32768 ) va = -32768;
        if ( vb > 32767 ) vb = 32767; if ( vb <= -32768 ) vb = -32768;
        *outp++ = (S16)va;
        *outp++ = (S16)vb;
    }
}

// DCT merge (inverse of split)
static void scalar_dct_merge_s16s(S16 out[], S16 left[], F32 scale, F32 const in[], UINTa N)
{
    UINTa N2 = N / 2;
    S16 *outp = out;
    F32 const *in0 = in;
    F32 const *in1 = in + N;

    for (UINTa k = 0; k < N2; ++k)
    {
        F32 a = *in0++;
        F32 b = *--in1;
        S32 va = (S32)(a*scale);
        S32 vb = (S32)(b*scale);
        if ( va > 32767 ) va = 32767; if ( va <= -32768 ) va = -32768;
        if ( vb > 32767 ) vb = 32767; if ( vb <= -32768 ) vb = -32768;
        *outp++ = *left++;
        *outp++ = (S16)va;
        *outp++ = *left++;
        *outp++ = (S16)vb;
    }
}

// DCT-II modulation step
static void scalar_dct2_modulate(F32 out[], F32 const in[], UINTa N, UINTa Nlast, rfft_complex const *twiddle)
{
    F32 const kSqrtOneHalf = 0.7071067811865475244f;

    // First and Nyquist buckets
    out[0] = in[0];
    out[N/2] = kSqrtOneHalf * in[1];

    // Rest
    for (UINTa k = 1; k < Nlast; ++k)
    {
        F32 Zr = in[k*2 + 0];
        F32 Zi = in[k*2 + 1];
        F32 wr = twiddle[k].re;
        F32 wi = twiddle[k].im;

        out[k]   = wr*Zr + wi*Zi;
        out[N-k] = wr*Zi - wi*Zr;
    }
}

// DCT-III modulation step
static void scalar_dct3_modulate(F32 out[], F32 const in[], UINTa N, UINTa Nlast, rfft_complex const *twiddle)
{
    F32 const kSqrtTwo = 1.4142135623730950488f;

    // First and Nyquist buckets
    out[0] = in[0] + in[0];
    out[1] = kSqrtTwo * in[N/2];

    // Rest
    for (UINTa k = 1; k < Nlast; ++k)
    {
        F32 Zr = in[k];
        F32 Zi = in[N-k];
        F32 wr = twiddle[k].re;
        F32 wi = twiddle[k].im;

        out[k*2 + 0] = wr*Zr - wi*Zi;
        out[k*2 + 1] = wr*Zi + wi*Zr;
    }
}

#if !defined(__RADNEON__) 

// Fully scalar kernel set is always an option
static KernelSet const s_kernel_scalar = {
    scalar_rfpost,
    scalar_ripre,
    scalar_cfpass,
    scalar_cipass,
    scalar_radix4,
    scalar_2radix2,
    scalar_dct_split,
    scalar_dct_merge,
    scalar_dct_merge_s16,
    scalar_dct_merge_s16s,
    scalar_dct2_modulate,
    scalar_dct3_modulate
};

#endif

// --------------------------------------------------------------------------
// SSE/SSE3 kernels for x86.
// --------------------------------------------------------------------------

#ifdef __RADX86__

#include <emmintrin.h> // we require SSE2.

#if defined( RAD_USES_SSE3 )

// this is just to compile with GCC 4.8 - there will be a warning but it works
//   for GCC 4.9 and clang, this isn't necessary
#ifdef __RAD_GCC_VERSION__
#if __RAD_GCC_VERSION__ < 40900
#pragma GCC push_options
#pragma GCC target("sse3")
#define __SSE3__
#endif
#endif

#include <pmmintrin.h>

// First time round: actual SSE3
#define RADFFT_SSE3_PREFIX(name) RAD_USES_SSE3 sse3_##name
#include "radfft_sse3.inl"
#undef RADFFT_SSE3_PREFIX

// this is just to compile with GCC 4.8 - there will be a warning but it works
//   for GCC 4.9 and clang, this isn't necessary
#ifdef __RAD_GCC_VERSION__
#if __RAD_GCC_VERSION__ < 40900
#pragma GCC pop_options
#endif
#endif

#endif // ifdef RADFFT_SSE3

#if !defined(RAD_GUARANTEED_SSE3)

// Second time round: plain SSE with these glorious hacks.
// This doesn't give optimal SSE code but it's not terrible either.

#define RADFFT_SSE3_PREFIX(name) sse_##name
#define _mm_moveldup_ps(x) _mm_shuffle_ps((x), (x), 0xa0)
#define _mm_movehdup_ps(x) _mm_shuffle_ps((x), (x), 0xf5)
#define _mm_addsub_ps(a,b) _mm_add_ps((a), _mm_xor_ps(b, _mm_setr_ps(-0.0f, 0.0f, -0.0f, 0.0f)))
#include "radfft_sse3.inl"
#undef _mm_moveldup_ps
#undef _mm_movehdup_ps
#undef _mm_addsub_ps
#undef RADFFT_SSE3_PREFIX

#endif

// SSE 2x Radix-2 FFT codelet. Self-inverse (up to scale).
static void sse_2radix2(rfft_complex out[], rfft_complex const in0[], rfft_complex const in1[], rfft_complex const in2[], rfft_complex const in3[], UINTa j0, UINTa j1, Index const *perm)
{
    if (j0 == j1)
        return;

    // SIMD loop wants an even number of elements at an aligned
    // offset. Thus, we may have extra elements at the beginning
    // or end.
    UINTa j1a = j1 & ~1;
    if (j0 & 1)
    {
        scalar_2radix2(out, in0, in1, in2, in3, j0, j0 + 1, perm);
        ++j0;
    }

    for (UINTa j = j0; j < j1; j += 2)
    {
        __m128 a = _mm_load_ps((const F32 *) &in0[j]);
        __m128 b = _mm_load_ps((const F32 *) &in2[j]);
        __m128 c = _mm_load_ps((const F32 *) &in3[j]);
        __m128 d = _mm_load_ps((const F32 *) &in1[j]);
        F32 *o0 = (F32 *) (out + perm[j+0]);
        F32 *o1 = (F32 *) (out + perm[j+1]);

        __m128 E = _mm_add_ps(a, b);
        __m128 F = _mm_sub_ps(a, b);
        __m128 G = _mm_add_ps(c, d);
        __m128 H = _mm_sub_ps(c, d);

        _mm_store_ps(o0 + 0, _mm_movelh_ps(E, F));
        _mm_store_ps(o1 + 0, _mm_movehl_ps(F, E));
        _mm_store_ps(o0 + 4, _mm_movelh_ps(G, H));
        _mm_store_ps(o1 + 4, _mm_movehl_ps(H, G));
    }

    if (j1a != j1)
        scalar_2radix2(out, in0, in1, in2, in3, j1a, j1, perm);
}

// SSE Radix-4 FFT codelet. To get the inverse (up to scale), swap in1 and in3.
static void sse_radix4(rfft_complex out[], rfft_complex const in0[], rfft_complex const in1[], rfft_complex const in2[], rfft_complex const in3[], UINTa j0, UINTa j1, Index const *perm)
{
    if (j0 == j1)
        return;

    // SIMD loop wants an even number of elements at an aligned
    // offset. Thus, we may have extra elements at the beginning
    // or end.
    UINTa j1a = j1 & ~1;
    if (j0 & 1)
    {
        scalar_radix4(out, in0, in1, in2, in3, j0, j0 + 1, perm);
        ++j0;
    }

    __m128 conjflip = _mm_setr_ps(0.0f, -0.0f, 0.0f, -0.0f);

    for (UINTa j = j0; j < j1a; j += 2)
    {
        __m128 a = _mm_load_ps((const F32 *) &in0[j]);
        __m128 b = _mm_load_ps((const F32 *) &in2[j]);
        __m128 c = _mm_load_ps((const F32 *) &in1[j]);
        __m128 d = _mm_load_ps((const F32 *) &in3[j]);
        F32 *o0 = &out[perm[j+0]].re;
        F32 *o1 = &out[perm[j+1]].re;

        __m128 A = _mm_add_ps(a, b);
        __m128 B = _mm_sub_ps(a, b);
        __m128 C = _mm_add_ps(c, d);
        __m128 D = _mm_sub_ps(c, d);

        // D *= -i
        D = _mm_xor_ps(_mm_shuffle_ps(D, D, 0xb1), conjflip);

        __m128 E = _mm_add_ps(A, C);
        __m128 F = _mm_add_ps(B, D);
        __m128 G = _mm_sub_ps(A, C);
        __m128 H = _mm_sub_ps(B, D);

        _mm_store_ps(o0 + 0, _mm_movelh_ps(E, F));
        _mm_store_ps(o1 + 0, _mm_movehl_ps(F, E));
        _mm_store_ps(o0 + 4, _mm_movelh_ps(G, H));
        _mm_store_ps(o1 + 4, _mm_movehl_ps(H, G));
    }

    if (j1a != j1)
        scalar_radix4(out, in0, in1, in2, in3, j1a, j1, perm);
}

static void sse_dct_split(F32 out[], F32 const in[], UINTa N)
{
#ifdef WANT_TINY
    if (N < 8)
    {
        scalar_dct_split(out, in, N);
        return;
    }
#endif

    // N is pow2 and >=8
    F32 *out0 = out;
    F32 *out1 = out + N;
    F32 const *inp = in;
    F32 const *inp_end = in + N;

    do
    {
        __m128 v0 = _mm_load_ps(inp);
        __m128 v1 = _mm_load_ps(inp + 4);
        __m128 s0 = _mm_shuffle_ps(v0, v1, 0x88); // x0,x2,x4,x6
        __m128 s1 = _mm_shuffle_ps(v1, v0, 0x77); // x7,x5,x3,x1

        _mm_store_ps(out0, s0);
        _mm_store_ps(out1 - 4, s1);

        inp += 8;
        out0 += 4;
        out1 -= 4;
    } while (inp != inp_end);
}

static void sse_dct_merge(F32 out[], F32 const in[], UINTa N)
{
#ifdef WANT_TINY
    if (N < 8)
    {
        scalar_dct_merge(out, in, N);
        return;
    }
#endif

    // N is pow2 and >=8
    F32 const *in0 = in;
    F32 const *in1 = in + N;
    F32 *outp = out;
    F32 *outp_end = out + N;

    do
    {
        __m128 i0 = _mm_load_ps(in0);     // x0,x2,x4,x6
        __m128 i1 = _mm_load_ps(in1 - 4); // x7,x5,x3,x1
        __m128 s1 = _mm_shuffle_ps(i1, i1, 0x1b); // x1,x3,x5,x7

        __m128 o0 = _mm_unpacklo_ps(i0, s1);
        __m128 o1 = _mm_unpackhi_ps(i0, s1);

        _mm_store_ps(outp, o0);
        _mm_store_ps(outp + 4, o1);
        outp += 8;
        in0 += 4;
        in1 -= 4;
    }
    while (outp != outp_end);
}

static void sse_dct_merge_s16(S16 out[], F32 scale, F32 const in[], UINTa N)
{
#ifdef WANT_TINY
    if (N < 8)
    {
        scalar_dct_merge_s16(out, scale, in, N);
        return;
    }
#endif

    // N is pow2 and >=8
    F32 const *in0 = in;
    F32 const *in1 = in + N;
    S16 *outp = out;
    S16 *outp_end = out + N;

    __m128 scale128 = _mm_load1_ps( (float*)(&scale) );

    do
    {
        __m128 i0 = _mm_load_ps(in0);       // x0,x2,x4,x6
        __m128 i1 = _mm_load_ps(in1 - 4);   // x7,x5,x3,x1
        __m128 s1 = _mm_shuffle_ps(i1, i1, 0x1b); // x1,x3,x5,x7

        __m128 o0 = _mm_unpacklo_ps(i0, s1);
        __m128 o1 = _mm_unpackhi_ps(i0, s1);

        o0 = _mm_mul_ps( o0, scale128 );
        o1 = _mm_mul_ps( o1, scale128 );

        // [ x, y, z, w ]
        __m128i io0 = _mm_cvtps_epi32( o0 );
        __m128i io1 = _mm_cvtps_epi32( o1 );

        __m128i p0 = _mm_packs_epi32( io0, io1 );

        _mm_store_si128((__m128i*)outp, p0);

        outp += 8;
        in0 += 4;
        in1 -= 4;
    }
    while (outp != outp_end);
}

static void sse_dct_merge_s16s(S16 out[], S16 left[], F32 scale, F32 const in[], UINTa N)
{
#ifdef WANT_TINY
    if (N < 8)
    {
        scalar_dct_merge_s16(out, scale, in, N);
        return;
    }
#endif

    // N is pow2 and >=8
    F32 const *in0 = in;
    F32 const *in1 = in + N;
    S16 *outp = out;
    S16 *outp_end = out + (N*2);

    __m128 scale128 = _mm_load1_ps( (float*)(&scale) );

    do
    {
        __m128 i0 = _mm_load_ps(in0);       // x0,x2,x4,x6
        __m128 i1 = _mm_load_ps(in1 - 4);   // x7,x5,x3,x1
        __m128 s1 = _mm_shuffle_ps(i1, i1, 0x1b); // x1,x3,x5,x7

        __m128 o0 = _mm_unpacklo_ps(i0, s1);
        __m128 o1 = _mm_unpackhi_ps(i0, s1);

        o0 = _mm_mul_ps( o0, scale128 );
        o1 = _mm_mul_ps( o1, scale128 );

        // [ x, y, z, w ]
        __m128i io0 = _mm_cvtps_epi32( o0 );
        __m128i io1 = _mm_cvtps_epi32( o1 );

        __m128i p0 = _mm_packs_epi32( io0, io1 );

        io1 = _mm_load_si128((__m128i*)left);

        io0 = _mm_unpacklo_epi16( io1, p0 );
        io1 = _mm_unpackhi_epi16( io1, p0 );
        
        _mm_store_si128((__m128i*)outp, io0);
        _mm_store_si128((__m128i*)(outp+8), io1);

        outp += 16;
        left += 8;
        in0 += 4;
        in1 -= 4;
    }
    while (outp != outp_end);
}

static void sse_dct3_modulate(F32 out[], F32 const in[], UINTa N, UINTa Nlast, rfft_complex const *twiddle)
{
#ifdef WANT_TINY
    if (N < 8)
    {
        scalar_dct3_modulate(out, in, N, Nlast, twiddle);
        return;
    }
#endif

    // First few bins have exceptional cases, let scalar routine handle it
    FFTASSERT((Nlast % 4) == 0 && Nlast >= 4);
    scalar_dct3_modulate(out, in, N, 4, twiddle);

    for (UINTa k = 4; k < Nlast; k += 4)
    {
        __m128 Zr   = _mm_load_ps(in + k);
        __m128 rZi  = _mm_loadu_ps(in + N - 3 - k); // reversed Zi
        __m128 Zi   = _mm_shuffle_ps(rZi, rZi, 0x1b); // reverse it to get values the right way around

        __m128 w0   = _mm_load_ps(&twiddle[k + 0].re);
        __m128 w1   = _mm_load_ps(&twiddle[k + 2].re);

        __m128 wr   = _mm_shuffle_ps(w0, w1, 0x88); // real parts of twiddles
        __m128 wi   = _mm_shuffle_ps(w0, w1, 0xdd); // imag parts of twiddles

        __m128 re   = _mm_sub_ps(_mm_mul_ps(wr, Zr), _mm_mul_ps(wi, Zi));
        __m128 im   = _mm_add_ps(_mm_mul_ps(wr, Zi), _mm_mul_ps(wi, Zr));

        _mm_store_ps(out + k*2 + 0, _mm_unpacklo_ps(re, im));
        _mm_store_ps(out + k*2 + 4, _mm_unpackhi_ps(re, im));
    }
}


#if !defined(RAD_GUARANTEED_SSE3)
// Kernels to use when SSE (but not SSE3) is available
static KernelSet const s_kernel_sse = {
    sse_rfpost,
    sse_ripre,
    sse_cfpass,
    sse_cipass,
    sse_radix4,
    sse_2radix2,
    sse_dct_split,
    sse_dct_merge,
    sse_dct_merge_s16,
    sse_dct_merge_s16s,
    sse_dct2_modulate,
    sse_dct3_modulate
};
#endif

#ifdef RAD_USES_SSE3

// Kernels to use when SSE3 is available
static KernelSet const s_kernel_sse3 = {
    sse3_rfpost,
    sse3_ripre,
    sse3_cfpass,
    sse3_cipass,
    sse_radix4,
    sse_2radix2,
    sse_dct_split,
    sse_dct_merge,
    sse_dct_merge_s16,
    sse_dct_merge_s16s,
    sse3_dct2_modulate,
    sse_dct3_modulate
};

#endif

#ifdef RADFFT_AVX

#include <immintrin.h>

// AVX complex forward conjugate split-radix reduction pass.
// This is literally sse3_cfpass copied, with "__m128" replaced with "__m256",
// "_mm_" replaced with "_mm256_" and the offset increments of "4" replaced with
// "8".
//
// The N1=2 special case needed to be added, but that's it.
static void avx_cfpass(rfft_complex data[], PlanElement const *plan, UINTa Nover4)
{
    __m256 conjflip = _mm256_setr_ps(0.0f, -0.0f, 0.0f, -0.0f, 0.0f, -0.0f, 0.0f, -0.0f);
    --plan;

    do
    {
        ++plan;
        UINTa N1 = plan->Nloop;
        UINTa step = N1 * 2;
        F32 *out = (F32 *) (data + plan->offs);
        F32 *out_end = out + step;

        F32 const *twiddle = (F32 const *) (s_twiddles + N1);
        if (N1 > 2)
        {
            do
            {
                __m256 Zk       = _mm256_load_ps(out + 2*step);
                __m256 Zpk      = _mm256_load_ps(out + 3*step);
                __m256 w        = _mm256_load_ps(twiddle);
                __m256 w_re     = _mm256_moveldup_ps(w);
                __m256 w_im     = _mm256_movehdup_ps(w);

                // Twiddle Zk, Z'k
                Zk  = _mm256_addsub_ps(_mm256_mul_ps(Zk, w_re), _mm256_mul_ps(_mm256_shuffle_ps(Zk, Zk, 0xb1 /* yxwz */), w_im));
                Zpk = _mm256_addsub_ps(_mm256_mul_ps(_mm256_shuffle_ps(Zpk, Zpk, 0xb1), w_re), _mm256_mul_ps(Zpk, w_im));

                __m256 Zsum = _mm256_add_ps(_mm256_shuffle_ps(Zpk, Zpk, 0xb1), Zk);
                __m256 Zdif = _mm256_sub_ps(_mm256_shuffle_ps(Zk, Zk, 0xb1), Zpk);

                // Even inputs
                __m256 Uk0      = _mm256_load_ps(out + 0*step);
                __m256 Uk1      = _mm256_load_ps(out + 1*step);

                // Output butterflies
                _mm256_store_ps(out + 0*step, _mm256_add_ps(Uk0,    Zsum));
                _mm256_store_ps(out + 1*step, _mm256_add_ps(Uk1,    _mm256_xor_ps(Zdif, conjflip)));
                _mm256_store_ps(out + 2*step, _mm256_sub_ps(Uk0,    Zsum));
                _mm256_store_ps(out + 3*step, _mm256_addsub_ps(Uk1, Zdif));

                out += 8;
                twiddle += 8;
            } while (out < out_end);
        }
        else
        {
            // N=2 (small case)
            __m128 Zk       = _mm_load_ps(out + 2*step);
            __m128 Zpk      = _mm_load_ps(out + 3*step);
            __m128 w        = _mm_load_ps(twiddle);
            __m128 w_re     = _mm_moveldup_ps(w);
            __m128 w_im     = _mm_movehdup_ps(w);

            // Twiddle Zk, Z'k
            Zk  = _mm_addsub_ps(_mm_mul_ps(Zk, w_re), _mm_mul_ps(_mm_shuffle_ps(Zk, Zk, 0xb1 /* yxwz */), w_im));
            Zpk = _mm_addsub_ps(_mm_mul_ps(_mm_shuffle_ps(Zpk, Zpk, 0xb1), w_re), _mm_mul_ps(Zpk, w_im));

            __m128 Zsum = _mm_add_ps(_mm_shuffle_ps(Zpk, Zpk, 0xb1), Zk);
            __m128 Zdif = _mm_sub_ps(_mm_shuffle_ps(Zk, Zk, 0xb1), Zpk);

            // Even inputs
            __m128 Uk0      = _mm_load_ps(out + 0*step);
            __m128 Uk1      = _mm_load_ps(out + 1*step);

            // Output butterflies
            _mm_store_ps(out + 0*step, _mm_add_ps(Uk0,    Zsum));
            _mm_store_ps(out + 1*step, _mm_add_ps(Uk1,    _mm_xor_ps(Zdif, _mm256_extractf128_ps(conjflip, 0))));
            _mm_store_ps(out + 2*step, _mm_sub_ps(Uk0,    Zsum));
            _mm_store_ps(out + 3*step, _mm_addsub_ps(Uk1, Zdif));
        }
    } while (plan->Nloop < Nover4);
}

// AVX complex inverse conjugate split-radix reduction pass. This is the main workhorse inner loop for IFFTs.
// This is literally sse3_cipass copied, with "__m128" replaced with "__m256",
// "_mm_" replaced with "_mm256_" and the offset increments of "4" replaced with
// "8".
//
// The N1=2 special case needed to be added, but that's it.
static void avx_cipass(rfft_complex data[], PlanElement const *plan, UINTa Nover4)
{
    __m256 conjflip = _mm256_setr_ps(0.0f, -0.0f, 0.0f, -0.0f, 0.0f, -0.0f, 0.0f, -0.0f);
    --plan;

    do
    {
        ++plan;
        UINTa N1 = plan->Nloop;
        UINTa step = N1 * 2;
        F32 *out = (F32 *) (data + plan->offs);
        F32 *out_end = out + step;

        F32 const *twiddle = (F32 const *) (s_twiddles + N1);
        if (N1 > 2)
        {
            do
            {
                __m256 Zk       = _mm256_load_ps(out + 2*step);
                __m256 Zpk      = _mm256_load_ps(out + 3*step);
                __m256 w        = _mm256_load_ps(twiddle);
                __m256 w_re     = _mm256_moveldup_ps(w);
                __m256 w_im     = _mm256_movehdup_ps(w);

                // Twiddle Zk, Z'k
                Zpk = _mm256_addsub_ps(_mm256_mul_ps(Zpk, w_re), _mm256_mul_ps(_mm256_shuffle_ps(Zpk, Zpk, 0xb1), w_im));
                Zk  = _mm256_addsub_ps(_mm256_mul_ps(_mm256_shuffle_ps(Zk, Zk, 0xb1), w_re), _mm256_mul_ps(Zk, w_im));

                __m256 Zsum = _mm256_add_ps(_mm256_shuffle_ps(Zk, Zk, 0xb1), Zpk);
                __m256 Zdif = _mm256_sub_ps(_mm256_shuffle_ps(Zpk, Zpk, 0xb1), Zk);

                // Even inputs
                __m256 Uk0      = _mm256_load_ps(out + 0*step);
                __m256 Uk1      = _mm256_load_ps(out + 1*step);

                // Output butterflies
                _mm256_store_ps(out + 0*step, _mm256_add_ps(Uk0,    Zsum));
                _mm256_store_ps(out + 1*step, _mm256_add_ps(Uk1,    _mm256_xor_ps(Zdif, conjflip)));
                _mm256_store_ps(out + 2*step, _mm256_sub_ps(Uk0,    Zsum));
                _mm256_store_ps(out + 3*step, _mm256_addsub_ps(Uk1, Zdif));

                out += 8;
                twiddle += 8;
            } while (out < out_end);
        }
        else
        {
            __m128 Zk       = _mm_load_ps(out + 2*step);
            __m128 Zpk      = _mm_load_ps(out + 3*step);
            __m128 w        = _mm_load_ps(twiddle);
            __m128 w_re     = _mm_moveldup_ps(w);
            __m128 w_im     = _mm_movehdup_ps(w);

            // Twiddle Zk, Z'k
            Zpk = _mm_addsub_ps(_mm_mul_ps(Zpk, w_re), _mm_mul_ps(_mm_shuffle_ps(Zpk, Zpk, 0xb1), w_im));
            Zk  = _mm_addsub_ps(_mm_mul_ps(_mm_shuffle_ps(Zk, Zk, 0xb1), w_re), _mm_mul_ps(Zk, w_im));

            __m128 Zsum = _mm_add_ps(_mm_shuffle_ps(Zk, Zk, 0xb1), Zpk);
            __m128 Zdif = _mm_sub_ps(_mm_shuffle_ps(Zpk, Zpk, 0xb1), Zk);

            // Even inputs
            __m128 Uk0      = _mm_load_ps(out + 0*step);
            __m128 Uk1      = _mm_load_ps(out + 1*step);

            // Output butterflies
            _mm_store_ps(out + 0*step, _mm_add_ps(Uk0,    Zsum));
            _mm_store_ps(out + 1*step, _mm_add_ps(Uk1,    _mm_xor_ps(Zdif, _mm256_extractf128_ps(conjflip, 0))));
            _mm_store_ps(out + 2*step, _mm_sub_ps(Uk0,    Zsum));
            _mm_store_ps(out + 3*step, _mm_addsub_ps(Uk1, Zdif));
        }
    } while (plan->Nloop < Nover4);
}

// Kernels to use when AVX is available
static KernelSet const s_kernel_avx = {
    sse3_rfpost,
    sse3_ripre,
    avx_cfpass,
    avx_cipass,
    sse_radix4,
    sse_2radix2,
    sse_dct_split,
    sse_dct_merge,
    sse_dct_merge_s16,
    sse_dct_merge_s16s,
    sse3_dct2_modulate,
    sse_dct3_modulate
};

#endif

#ifdef _MSC_VER

#include <intrin.h>

#else

// Assume GCC-like and try to provide MSVC-style __cpuid
static void __cpuid(int *info, int which)
{
    void *saved=0;

    // NOTE(fg): With PIC on Mac, can't overwrite ebx, so we
    // need to play it safe.
#ifdef __RAD64__
    __asm__ __volatile__ (
        "movq %%rbx, %5\n"
        "cpuid\n"
        "movl %%ebx, %1\n"
        "movq %5, %%rbx"
        : "=a" (info[0]), 
          "=r" (info[1]),
          "=c" (info[2]),
          "=d" (info[3])
        : "a" (which),
          "m" (saved)
        : "cc");
#else
    __asm__ __volatile__ (
        "movl %%ebx, %5\n"
        "cpuid\n"
        "movl %%ebx, %1\n"
        "movl %5, %%ebx"
        : "=a" (info[0]), 
          "=r" (info[1]),
          "=c" (info[2]),
          "=d" (info[3])
        : "a"(which),
          "m" (saved)
        : "cc");
#endif
}

#endif

static KernelSet const *x86_select_kernels()
{
#if defined(__RADCONSOLE__)
     return &s_kernel_avx;
#else
    // query features
    int info[4];
    __cpuid(info, 1);

#ifdef RADFFT_AVX
    if (info[2] & (1u << 28)) // AVX available? (bit 28 in ECX)
        return &s_kernel_avx;
#endif

#if defined(RAD_GUARANTEED_SSE3)
    return &s_kernel_sse3;
#else

#ifdef RAD_USES_SSE3
    if (info[2] & (1u << 0)) // SSE3 available? (bit 0 in ECX)
        return &s_kernel_sse3;
#endif

#ifdef INC_BINK2
    return &s_kernel_sse;  // bink2 requires minimum of SSE2
#else
    if (info[3] & (1u << 25)) // SSE available? (bit 25 in EDX)
        return &s_kernel_sse;

    return &s_kernel_scalar;
#endif
#endif
#endif
}

#define CHOOSE_KERNELS x86_select_kernels()

#endif // __RADX86__

// --------------------------------------------------------------------------
// NEON kernels for ARM.
// --------------------------------------------------------------------------

#ifdef __RADNEON__

#include <arm_neon.h>

static inline float32x4_t neon_reverse(float32x4_t v)
{
    // Swap low/high halves
    // Since VREV is on d registers, ideally this just copy-propagates away
    v = vcombine_f32(vget_high_f32(v), vget_low_f32(v));
    return vrev64q_f32(v);
}

// NEON real FFT post-pass
static void neon_rfpost(rfft_complex out[], UINTa kEnd, UINTa N4)
{
    if (N4 < 4)
    {
        scalar_rfpost(out, N4, N4);
        return;
    }

    // Handle first few bins scalar.
    scalar_rfpost(out, 4, N4);

    F32 const *twiddle = (F32 const *) (s_twiddles + (N4 + 4));
    F32 *out0 = (F32 *) (out + 4);
    F32 *out1 = (F32 *) (out + (N4 * 2) - 7);
    F32 *out0_end = (F32 *) (out + kEnd);

    while (out0 < out0_end)
    {
        float32x4x2_t w = vld2q_f32(twiddle);
        float32x4x2_t i0 = vld2q_f32(out0);
        float32x4x2_t i1 = vld2q_f32(out1);

        // Reverse i1
        i1.val[0] = neon_reverse(i1.val[0]);
        i1.val[1] = neon_reverse(i1.val[1]);

        float32x4_t dr = vmulq_n_f32(vsubq_f32(i1.val[0], i0.val[0]), 0.5f);
        float32x4_t di = vmulq_n_f32(vaddq_f32(i1.val[1], i0.val[1]), 0.5f);
        float32x4_t evr = vaddq_f32(i0.val[0], dr);
        float32x4_t evi = vsubq_f32(i0.val[1], di);
        float32x4_t odr = vaddq_f32(vmulq_f32(w.val[0], di), vmulq_f32(w.val[1], dr));
        float32x4_t odi = vsubq_f32(vmulq_f32(w.val[0], dr), vmulq_f32(w.val[1], di));

        float32x4x2_t o0, o1;
        o0.val[0] = vaddq_f32(evr, odr);
        o0.val[1] = vaddq_f32(evi, odi);
        o1.val[0] = vsubq_f32(evr, odr);
        o1.val[1] = vsubq_f32(odi, evi);
        
        o1.val[0] = neon_reverse(o1.val[0]);
        o1.val[1] = neon_reverse(o1.val[1]);
        vst2q_f32(out0, o0);
        vst2q_f32(out1, o1);

        out0 += 8;
        out1 -= 8;
        twiddle += 8;
    }
}

// NEON real IFFT pre-pass
static void neon_ripre(rfft_complex out[], UINTa kEnd, UINTa N4)
{
    if (N4 < 4)
    {
        scalar_ripre(out, N4, N4);
        return;
    }

    // Handle first few bins scalar.
    scalar_ripre(out, 4, N4);

    F32 const *twiddle = (F32 const *) (s_twiddles + (N4 + 4));
    F32 *out0 = (F32 *) (out + 4);
    F32 *out1 = (F32 *) (out + (N4 * 2) - 7);
    F32 *out0_end = (F32 *) (out + kEnd);

    while (out0 < out0_end)
    {
        float32x4x2_t w  = vld2q_f32(twiddle);
        float32x4x2_t i0 = vld2q_f32(out0);
        float32x4x2_t i1 = vld2q_f32(out1);

        // Reverse i1
        i1.val[0] = neon_reverse(i1.val[0]);
        i1.val[1] = neon_reverse(i1.val[1]);

        float32x4_t dr = vmulq_n_f32(vsubq_f32(i0.val[0], i1.val[0]), 0.5f);
        float32x4_t di = vmulq_n_f32(vaddq_f32(i0.val[1], i1.val[1]), 0.5f);
        float32x4_t evr = vsubq_f32(i0.val[0], dr);
        float32x4_t evi = vsubq_f32(i0.val[1], di);
        float32x4_t odr = vaddq_f32(vmulq_f32(w.val[0], di), vmulq_f32(w.val[1], dr));
        float32x4_t odi = vsubq_f32(vmulq_f32(w.val[0], dr), vmulq_f32(w.val[1], di));

        float32x4x2_t o0, o1;
        o0.val[0] = vsubq_f32(evr, odr);
        o0.val[1] = vaddq_f32(evi, odi);
        o1.val[0] = vaddq_f32(evr, odr);
        o1.val[1] = vsubq_f32(odi, evi);
        
        o1.val[0] = neon_reverse(o1.val[0]);
        o1.val[1] = neon_reverse(o1.val[1]);
        vst2q_f32(out0, o0);
        vst2q_f32(out1, o1);

        out0 += 8;
        out1 -= 8;
        twiddle += 8;
    }
}

// NEON complex conjugate split-radix reduction pass. This is the main workhorse inner loop for IFFTs.
static void neon_cfpass(rfft_complex data[], PlanElement const *plan, UINTa Nover4)
{
    UINTa N1;

    do
    {
        N1 = plan->Nloop;
        UINTa counter = N1/2;
        float *out0 = (float *) (data + plan->offs);
        float *out1 = out0 + 2*N1;
        float *out2 = out0 + 4*N1;
        float *out3 = out0 + 6*N1;
        float const *twiddle = (float const *) (s_twiddles + N1);
        ++plan;

        if (counter > 1)
        {
            do
            {
                // Load input complex values, unpacking as we go
                float32x4x2_t w  = vld2q_f32(twiddle);
                float32x4x2_t x2 = vld2q_f32(out2);
                float32x4x2_t x3 = vld2q_f32(out3);

                // This is a straight translation of the scalar code
                float32x4_t Zkr  = vsubq_f32(vmulq_f32(w.val[0], x2.val[0]), vmulq_f32(w.val[1], x2.val[1]));
                float32x4_t Zki  = vaddq_f32(vmulq_f32(w.val[0], x2.val[1]), vmulq_f32(w.val[1], x2.val[0]));
                float32x4_t Zpkr = vaddq_f32(vmulq_f32(w.val[0], x3.val[0]), vmulq_f32(w.val[1], x3.val[1]));
                float32x4_t Zpki = vsubq_f32(vmulq_f32(w.val[0], x3.val[1]), vmulq_f32(w.val[1], x3.val[0]));

                float32x4_t Zsumr = vaddq_f32(Zkr, Zpkr);
                float32x4_t Zsumi = vaddq_f32(Zki, Zpki);
                float32x4_t Zdifr = vsubq_f32(Zki, Zpki);
                float32x4_t Zdifi = vsubq_f32(Zpkr, Zkr);

                float32x4x2_t x0 = vld2q_f32(out0);
                float32x4x2_t x1 = vld2q_f32(out1);

                x2.val[0] = vsubq_f32(x0.val[0], Zsumr);
                x2.val[1] = vsubq_f32(x0.val[1], Zsumi);
                x0.val[0] = vaddq_f32(x0.val[0], Zsumr);
                x0.val[1] = vaddq_f32(x0.val[1], Zsumi);
                x3.val[0] = vsubq_f32(x1.val[0], Zdifr);
                x3.val[1] = vsubq_f32(x1.val[1], Zdifi);
                x1.val[0] = vaddq_f32(x1.val[0], Zdifr);
                x1.val[1] = vaddq_f32(x1.val[1], Zdifi);

                // Store back
                vst2q_f32(out0, x0);
                vst2q_f32(out1, x1);
                vst2q_f32(out2, x2);
                vst2q_f32(out3, x3);

                out0 += 8;
                out1 += 8;
                out2 += 8;
                out3 += 8;
                twiddle += 8;
            } while (counter -= 2);
        }
        else
        {
            // Load input complex values, unpacking as we go
            float32x2x2_t w  = vld2_f32(twiddle);
            float32x2x2_t x2 = vld2_f32(out2);
            float32x2x2_t x3 = vld2_f32(out3);

            // This is a straight translation of the scalar code
            float32x2_t Zkr  = vsub_f32(vmul_f32(w.val[0], x2.val[0]), vmul_f32(w.val[1], x2.val[1]));
            float32x2_t Zki  = vadd_f32(vmul_f32(w.val[0], x2.val[1]), vmul_f32(w.val[1], x2.val[0]));
            float32x2_t Zpkr = vadd_f32(vmul_f32(w.val[0], x3.val[0]), vmul_f32(w.val[1], x3.val[1]));
            float32x2_t Zpki = vsub_f32(vmul_f32(w.val[0], x3.val[1]), vmul_f32(w.val[1], x3.val[0]));

            float32x2_t Zsumr = vadd_f32(Zkr, Zpkr);
            float32x2_t Zsumi = vadd_f32(Zki, Zpki);
            float32x2_t Zdifr = vsub_f32(Zki, Zpki);
            float32x2_t Zdifi = vsub_f32(Zpkr, Zkr);

            float32x2x2_t x0 = vld2_f32(out0);
            float32x2x2_t x1 = vld2_f32(out1);

            x2.val[0] = vsub_f32(x0.val[0], Zsumr);
            x2.val[1] = vsub_f32(x0.val[1], Zsumi);
            x0.val[0] = vadd_f32(x0.val[0], Zsumr);
            x0.val[1] = vadd_f32(x0.val[1], Zsumi);
            x3.val[0] = vsub_f32(x1.val[0], Zdifr);
            x3.val[1] = vsub_f32(x1.val[1], Zdifi);
            x1.val[0] = vadd_f32(x1.val[0], Zdifr);
            x1.val[1] = vadd_f32(x1.val[1], Zdifi);

            // Store back
            vst2_f32(out0, x0);
            vst2_f32(out1, x1);
            vst2_f32(out2, x2);
            vst2_f32(out3, x3);
        }
    } while (N1 < Nover4);
}

// NEON complex inverse conjugate split-radix reduction pass. This is the main workhorse inner loop for IFFTs.
static void neon_cipass(rfft_complex data[], PlanElement const *plan, UINTa Nover4)
{
    UINTa N1;

    do
    {
        N1 = plan->Nloop;
        UINTa counter = N1 / 2;
        float *out0 = (float *) (data + plan->offs);
        float *out1 = out0 + 2*N1;
        float *out2 = out0 + 4*N1;
        float *out3 = out0 + 6*N1;
        float const *twiddle = (float const *) (s_twiddles + N1);
        ++plan;

        if (counter > 1)
        {
            do
            {
                // Load input complex values, unpacking as we go
                float32x4x2_t w  = vld2q_f32(twiddle);
                float32x4x2_t x2 = vld2q_f32(out2);
                float32x4x2_t x3 = vld2q_f32(out3);

                // This is a straight translation of the scalar code
                float32x4_t Zkr  = vaddq_f32(vmulq_f32(w.val[0], x2.val[0]), vmulq_f32(w.val[1], x2.val[1]));
                float32x4_t Zpkr = vsubq_f32(vmulq_f32(w.val[0], x3.val[0]), vmulq_f32(w.val[1], x3.val[1]));
                float32x4_t Zki  = vsubq_f32(vmulq_f32(w.val[0], x2.val[1]), vmulq_f32(w.val[1], x2.val[0]));
                float32x4_t Zpki = vaddq_f32(vmulq_f32(w.val[0], x3.val[1]), vmulq_f32(w.val[1], x3.val[0]));

                float32x4_t Zsumr = vaddq_f32(Zkr, Zpkr);
                float32x4_t Zdifi = vsubq_f32(Zkr, Zpkr);
                float32x4_t Zsumi = vaddq_f32(Zki, Zpki);
                float32x4_t Zdifr = vsubq_f32(Zpki, Zki);

                float32x4x2_t x0 = vld2q_f32(out0);
                float32x4x2_t x1 = vld2q_f32(out1);

                x2.val[0] = vsubq_f32(x0.val[0], Zsumr);
                x0.val[0] = vaddq_f32(x0.val[0], Zsumr);
                x3.val[1] = vsubq_f32(x1.val[1], Zdifi);
                x1.val[1] = vaddq_f32(x1.val[1], Zdifi);
                x2.val[1] = vsubq_f32(x0.val[1], Zsumi);
                x0.val[1] = vaddq_f32(x0.val[1], Zsumi);
                x3.val[0] = vsubq_f32(x1.val[0], Zdifr);
                x1.val[0] = vaddq_f32(x1.val[0], Zdifr);

                // Store back
                vst2q_f32(out2, x2);
                vst2q_f32(out0, x0);
                vst2q_f32(out3, x3);
                vst2q_f32(out1, x1);

                out0 += 8;
                out1 += 8;
                out2 += 8;
                out3 += 8;
                twiddle += 8;
            } while (counter -= 2);
        }
        else
        {
            // Load input complex values, unpacking as we go
            float32x2x2_t w  = vld2_f32(twiddle);
            float32x2x2_t x2 = vld2_f32(out2);
            float32x2x2_t x3 = vld2_f32(out3);

            // This is a straight translation of the scalar code
            float32x2_t Zkr  = vadd_f32(vmul_f32(w.val[0], x2.val[0]), vmul_f32(w.val[1], x2.val[1]));
            float32x2_t Zpkr = vsub_f32(vmul_f32(w.val[0], x3.val[0]), vmul_f32(w.val[1], x3.val[1]));
            float32x2_t Zki  = vsub_f32(vmul_f32(w.val[0], x2.val[1]), vmul_f32(w.val[1], x2.val[0]));
            float32x2_t Zpki = vadd_f32(vmul_f32(w.val[0], x3.val[1]), vmul_f32(w.val[1], x3.val[0]));

            float32x2_t Zsumr = vadd_f32(Zkr, Zpkr);
            float32x2_t Zdifi = vsub_f32(Zkr, Zpkr);
            float32x2_t Zsumi = vadd_f32(Zki, Zpki);
            float32x2_t Zdifr = vsub_f32(Zpki, Zki);

            float32x2x2_t x0 = vld2_f32(out0);
            float32x2x2_t x1 = vld2_f32(out1);

            x2.val[0] = vsub_f32(x0.val[0], Zsumr);
            x0.val[0] = vadd_f32(x0.val[0], Zsumr);
            x3.val[1] = vsub_f32(x1.val[1], Zdifi);
            x1.val[1] = vadd_f32(x1.val[1], Zdifi);
            x2.val[1] = vsub_f32(x0.val[1], Zsumi);
            x0.val[1] = vadd_f32(x0.val[1], Zsumi);
            x3.val[0] = vsub_f32(x1.val[0], Zdifr);
            x1.val[0] = vadd_f32(x1.val[0], Zdifr);

            // Store back
            vst2_f32(out2, x2);
            vst2_f32(out0, x0);
            vst2_f32(out3, x3);
            vst2_f32(out1, x1);
        }
    } while (N1 < Nover4);
}

// NEON 2x Radix-2 FFT codelet.
static void neon_2radix2(rfft_complex out[], rfft_complex const in0[], rfft_complex const in1[], rfft_complex const in2[], rfft_complex const in3[], UINTa j0, UINTa j1, Index const *perm)
{
    if (j0 == j1)
        return;

    // SIMD loop wants an even number of elements at an aligned
    // offset. Thus, we may have extra elements at the beginning
    // or end.
    UINTa j1a = j1 & ~1;
    if (j0 & 1)
    {
        scalar_2radix2(out, in0, in1, in2, in3, j0, j0 + 1, perm);
        ++j0;
    }

    for (UINTa j = j0; j < j1a; j += 2)
    {
        float32x4_t a = vld1q_f32(&in0[j].re);
        float32x4_t b = vld1q_f32(&in2[j].re);
        float32x4_t c = vld1q_f32(&in3[j].re);
        float32x4_t d = vld1q_f32(&in1[j].re);
        F32 *o0 = &out[perm[j+0]].re;
        F32 *o1 = &out[perm[j+1]].re;

        float32x4_t E = vaddq_f32(a, b);
        float32x4_t F = vsubq_f32(a, b);
        float32x4_t G = vaddq_f32(c, d);
        float32x4_t H = vsubq_f32(c, d);

        // Store
        vst1_f32(o0 + 0, vget_low_f32(E));
        vst1_f32(o0 + 2, vget_low_f32(F));
        vst1_f32(o0 + 4, vget_low_f32(G));
        vst1_f32(o0 + 6, vget_low_f32(H));
        vst1_f32(o1 + 0, vget_high_f32(E));
        vst1_f32(o1 + 2, vget_high_f32(F));
        vst1_f32(o1 + 4, vget_high_f32(G));
        vst1_f32(o1 + 6, vget_high_f32(H));
    }

    if (j1a != j1)
        scalar_2radix2(out, in0, in1, in2, in3, j1a, j1, perm);
}

// NEON Radix-4 FFT codelet.
static void neon_radix4(rfft_complex out[], rfft_complex const in0[], rfft_complex const in1[], rfft_complex const in2[], rfft_complex const in3[], UINTa j0, UINTa j1, Index const *perm)
{
    if (j0 == j1)
        return;

    // SIMD loop wants an even number of elements at an aligned
    // offset. Thus, we may have extra elements at the beginning
    // or end.
    UINTa j1a = j1 & ~1;
    if (j0 & 1)
    {
        scalar_radix4(out, in0, in1, in2, in3, j0, j0 + 1, perm);
        ++j0;
    }

    // NOTE: pretty un-NEON to work with interleaved values like that, but
    // it ends up better than the split variants I could come up with.
    U32 conjflip_c[4] = { 0, 0x80000000u, 0, 0x80000000u };
    uint32x4_t conjflip = vld1q_u32(conjflip_c);

    for (UINTa j = j0; j < j1a; j += 2)
    {
        float32x4_t a = vld1q_f32(&in0[j].re);
        float32x4_t b = vld1q_f32(&in2[j].re);
        float32x4_t c = vld1q_f32(&in1[j].re);
        float32x4_t d = vld1q_f32(&in3[j].re);
        F32 *o0 = &out[perm[j+0]].re;
        F32 *o1 = &out[perm[j+1]].re;

        float32x4_t A = vaddq_f32(a, b);
        float32x4_t B = vsubq_f32(a, b);
        float32x4_t C = vaddq_f32(c, d);
        float32x4_t D = vsubq_f32(c, d);

        // D *= -i
        D = vreinterpretq_f32_u32(veorq_u32(vreinterpretq_u32_f32(vrev64q_f32(D)), conjflip));

        float32x4_t E = vaddq_f32(A, C);
        float32x4_t F = vaddq_f32(B, D);
        float32x4_t G = vsubq_f32(A, C);
        float32x4_t H = vsubq_f32(B, D);

        // Store
        vst1_f32(o0 + 0, vget_low_f32(E));
        vst1_f32(o0 + 2, vget_low_f32(F));
        vst1_f32(o0 + 4, vget_low_f32(G));
        vst1_f32(o0 + 6, vget_low_f32(H));
        vst1_f32(o1 + 0, vget_high_f32(E));
        vst1_f32(o1 + 2, vget_high_f32(F));
        vst1_f32(o1 + 4, vget_high_f32(G));
        vst1_f32(o1 + 6, vget_high_f32(H));
    }

    if (j1a != j1)
        scalar_radix4(out, in0, in1, in2, in3, j1a, j1, perm);
}

static void neon_dct_split(F32 out[], F32 const in[], UINTa N)
{
#ifdef WANT_TINY
    if (N < 16)
    {
        scalar_dct_split(out, in, N);
        return;
    }
#endif

    // N is pow2 and >=16
    F32 *out0 = out;
    F32 *out1 = out + N;
    F32 const *inp = in;

    do
    {
        float32x4x2_t v0 = vld2q_f32(inp);
        float32x4x2_t v1 = vld2q_f32(inp + 8);

        vst1q_f32(out0, v0.val[0]);
        vst1q_f32(out0 + 4, v1.val[0]);
        vst1q_f32(out1 - 4, neon_reverse(v0.val[1])); 
        vst1q_f32(out1 - 8, neon_reverse(v1.val[1])); 

        inp += 16;
        out0 += 8;
        out1 -= 8;
    } while (N -= 16);
}

static void neon_dct_merge(F32 out[], F32 const in[], UINTa N)
{
#ifdef WANT_TINY
    if (N < 16)
    {
        scalar_dct_merge(out, in, N);
        return;
    }
#endif

    // N is pow2 and >=16
    F32 const *in0 = in;
    F32 const *in1 = in + N;
    F32 *outp = out;

    do
    {
        float32x4x2_t v0, v1;
        v0.val[0] = vld1q_f32(in0);
        v0.val[1] = neon_reverse(vld1q_f32(in1 - 4));
        v1.val[0] = vld1q_f32(in0 + 4);
        v1.val[1] = neon_reverse(vld1q_f32(in1 - 8));

        vst2q_f32(outp, v0);
        vst2q_f32(outp + 8, v1);

        outp += 16;
        in0 += 8;
        in1 -= 8;
    }
    while (N -= 16);
}


static void neon_dct_merge_s16(S16 out[], F32 scale, F32 const in[], UINTa N)
{
#ifdef WANT_TINY
    if (N < 16)
    {
        scalar_dct_merge_s16(out, scale, in, N);
        return;
    }
#endif

    // N is pow2 and >=16
    F32 const *in0 = in;
    F32 const *in1 = in + N;
    S16 *outp = out;

    float32x4_t scale128 = vmovq_n_f32( scale );

    do
    {
        float32x4x2_t v0, v1;
        int32x4x2_t i0, i1;
        int16x8x2_t is16;

        v0.val[0] = vld1q_f32(in0);
        v0.val[1] = vld1q_f32(in0 + 4);
        v1.val[0] = neon_reverse(vld1q_f32(in1 - 4));
        v1.val[1] = neon_reverse(vld1q_f32(in1 - 8));

        // scale the values
        v0.val[0] = vmulq_f32( v0.val[0], scale128 );
        v0.val[1] = vmulq_f32( v0.val[1], scale128 );
        v1.val[0] = vmulq_f32( v1.val[0], scale128 );
        v1.val[1] = vmulq_f32( v1.val[1], scale128 );
        
        // convert to 32-bit ints
        i0.val[0] = vcvtq_s32_f32( v0.val[0] );
        i0.val[1] = vcvtq_s32_f32( v0.val[1] );
        i1.val[0] = vcvtq_s32_f32( v1.val[0] );
        i1.val[1] = vcvtq_s32_f32( v1.val[1] );

        // merge them
        is16.val[0] = vcombine_s16(vqmovn_s32(i0.val[0]), vqmovn_s32(i0.val[1]));
        is16.val[1] = vcombine_s16(vqmovn_s32(i1.val[0]), vqmovn_s32(i1.val[1]));

        // store if
        vst2q_s16(outp, is16);

        outp += 16;
        in0 += 8;
        in1 -= 8;
    }
    while (N -= 16);
}

static void neon_dct_merge_s16s(S16 out[], S16 left[], F32 scale, F32 const in[], UINTa N)
{
#ifdef WANT_TINY
    if (N < 16)
    {
        scalar_dct_merge_s16s(out, left, scale, in, N);
        return;
    }
#endif

    // N is pow2 and >=16
    F32 const *in0 = in;
    F32 const *in1 = in + N;
    S16 *outp = out;

    float32x4_t scale128 = vmovq_n_f32( scale );

    do
    {
        float32x4x2_t v0, v1;
        int32x4x2_t i0, i1;
        int16x8x2_t rs16;
        int16x8x2_t ls16;
        int16x8x4_t zs16;

        v0.val[0] = vld1q_f32(in0);
        v0.val[1] = vld1q_f32(in0 + 4);
        v1.val[0] = neon_reverse(vld1q_f32(in1 - 4));
        v1.val[1] = neon_reverse(vld1q_f32(in1 - 8));

        // scale the values
        v0.val[0] = vmulq_f32( v0.val[0], scale128 );
        v0.val[1] = vmulq_f32( v0.val[1], scale128 );
        v1.val[0] = vmulq_f32( v1.val[0], scale128 );
        v1.val[1] = vmulq_f32( v1.val[1], scale128 );
        
        // convert to 32-bit ints
        i0.val[0] = vcvtq_s32_f32( v0.val[0] );
        i0.val[1] = vcvtq_s32_f32( v0.val[1] );
        i1.val[0] = vcvtq_s32_f32( v1.val[0] );
        i1.val[1] = vcvtq_s32_f32( v1.val[1] );

        // merge them
        rs16.val[0] = vcombine_s16(vqmovn_s32(i0.val[0]), vqmovn_s32(i0.val[1]));
        rs16.val[1] = vcombine_s16(vqmovn_s32(i1.val[0]), vqmovn_s32(i1.val[1]));

        ls16 = vld2q_s16( left );

        zs16.val[0] = ls16.val[0];
        zs16.val[1] = rs16.val[0];
        zs16.val[2] = ls16.val[1];
        zs16.val[3] = rs16.val[1];

        vst4q_s16(outp, zs16);

        left += 16;
        outp += 32;
        in0 += 8;
        in1 -= 8;
    }
    while (N -= 16);
}


static void neon_dct2_modulate(F32 out[], F32 const in[], UINTa N, UINTa Nlast, rfft_complex const *twiddle)
{
    if (N < 16)
    {
        scalar_dct2_modulate(out, in, N, Nlast, twiddle);
        return;
    }

    // First few bins have exceptional cases, let scalar routine handle it
    FFTASSERT((Nlast % 8) == 0 && Nlast >= 8);
    scalar_dct2_modulate(out, in, N, 8, twiddle);

    for (UINTa k = 8; k < Nlast; k += 8)
    {
        float32x4x2_t   Z0 = vld2q_f32(in + k*2);
        float32x4x2_t   Z1 = vld2q_f32(in + k*2 + 8);
        float32x4x2_t   w0 = vld2q_f32(&twiddle[k].re);
        float32x4x2_t   w1 = vld2q_f32(&twiddle[k + 4].re);

        float32x4_t     x0r = vaddq_f32(vmulq_f32(w0.val[0], Z0.val[0]), vmulq_f32(w0.val[1], Z0.val[1]));
        float32x4_t     x0i = vsubq_f32(vmulq_f32(w0.val[0], Z0.val[1]), vmulq_f32(w0.val[1], Z0.val[0]));
        float32x4_t     x1r = vaddq_f32(vmulq_f32(w1.val[0], Z1.val[0]), vmulq_f32(w1.val[1], Z1.val[1]));
        float32x4_t     x1i = vsubq_f32(vmulq_f32(w1.val[0], Z1.val[1]), vmulq_f32(w1.val[1], Z1.val[0]));

        x0i = neon_reverse(x0i);
        x1i = neon_reverse(x1i);

        vst1q_f32(out + k, x0r);
        vst1q_f32(out + k + 4, x1r);
        vst1q_f32(out + N - 3 - k, x0i);
        vst1q_f32(out + N - 7 - k, x1i);
    }
}

static void neon_dct3_modulate(F32 out[], F32 const in[], UINTa N, UINTa Nlast, rfft_complex const *twiddle)
{
    if (N < 16)
    {
        scalar_dct3_modulate(out, in, N, Nlast, twiddle);
        return;
    }

    // First few bins have exceptional cases, let scalar routine handle it
    FFTASSERT((Nlast % 8) == 0 && Nlast >= 8);
    scalar_dct3_modulate(out, in, N, 8, twiddle);

    for (UINTa k = 8; k < Nlast; k += 8)
    {
        float32x4_t     x0r = vld1q_f32(in + k);
        float32x4_t     x0i = vld1q_f32(in + N - 3 - k);
        float32x4_t     x1r = vld1q_f32(in + k + 4);
        float32x4_t     x1i = vld1q_f32(in + N - 7 - k);
        float32x4x2_t   w0 = vld2q_f32(&twiddle[k].re);
        float32x4x2_t   w1 = vld2q_f32(&twiddle[k + 4].re);

        // Reverse xi
        x0i = neon_reverse(x0i);
        x1i = neon_reverse(x1i);


        float32x4x2_t Z0, Z1;
        Z0.val[0] = vsubq_f32(vmulq_f32(w0.val[0], x0r), vmulq_f32(w0.val[1], x0i));
        Z0.val[1] = vaddq_f32(vmulq_f32(w0.val[1], x0r), vmulq_f32(w0.val[0], x0i));
        Z1.val[0] = vsubq_f32(vmulq_f32(w1.val[0], x1r), vmulq_f32(w1.val[1], x1i));
        Z1.val[1] = vaddq_f32(vmulq_f32(w1.val[1], x1r), vmulq_f32(w1.val[0], x1i));

        vst2q_f32(out + k*2, Z0);
        vst2q_f32(out + k*2 + 8, Z1);
    }
}

// Kernels to use when NEON is available
static KernelSet const s_kernel_neon = {
    neon_rfpost,
    neon_ripre,
    neon_cfpass,
    neon_cipass,
    neon_radix4,
    neon_2radix2,
    neon_dct_split,
    neon_dct_merge,
    neon_dct_merge_s16,
    neon_dct_merge_s16s,
    neon_dct2_modulate,
    neon_dct3_modulate
};

#define STATIC_KERNEL &s_kernel_neon

#endif


// --------------------------------------------------------------------------
// Driver/glue layer
// --------------------------------------------------------------------------

static void fft_driver(rfft_complex outc[], UINTa N, CFFTKernel *kernel)
{
    UINTa N1 = N/4;
    PlanElement local;
    PlanElement const *plan = s_recursion_plan;

    if (N > kMaxPlan)
    {
        // We're gonna call the kernel once after we're done recursing
        // to combine partial results; set that up here.
        local.offs = 0;
        local.Nloop = (Index)N1;
        plan = &local;

        // Recursion pattern for our conjugate split-radix FFT
        fft_driver(outc, N1*2, kernel);
        fft_driver(outc + N1*2, N1, kernel);
        fft_driver(outc + N1*3, N1, kernel);
    }

    kernel(outc, plan, N1);
}

static void fft_base_driver(rfft_complex out[], rfft_complex const in[], UINTa N, bool inverse, BaseKernel *kern_radix4, BaseKernel *kern_2radix2)
{
    // Handle base cases and permutation
    UINTa N1 = N / 4;
    U16 const *delta = s_permute + (N / kLeafN);
    rfft_complex const *in0 = in + 0*N1;
    rfft_complex const *in1 = in + 1*N1;
    rfft_complex const *in2 = in + 2*N1;
    rfft_complex const *in3 = in + 3*N1;

    UINTa j1 = N1 / 3 + 1;
    UINTa j2 = N1 - j1 + 1;

    if (!inverse)
    {
        kern_radix4 (out, in0, in1, in2, in3,  0, j1, delta);
        kern_2radix2(out, in0, in1, in2, in3, j1, j2, delta);
        kern_radix4 (out, in3, in0, in1, in2, j2, N1, delta);
    }
    else
    {
        kern_radix4 (out, in0, in3, in2, in1,  0, j1, delta);
        kern_2radix2(out, in0, in1, in2, in3, j1, j2, delta);
        kern_radix4 (out, in3, in2, in1, in0, j2, N1, delta);
    }
}

// --------------------------------------------------------------------------
// API / frontend layer
// --------------------------------------------------------------------------

// If we have no kernel set defined, default to everything scalar
// (this is for platforms we don't have optimizations for)
#if !defined(CHOOSE_KERNELS) && !defined(STATIC_KERNEL)
#define STATIC_KERNEL &s_kernel_scalar
#endif

#ifdef STATIC_KERNEL
#define s_kernel (STATIC_KERNEL)
#else
static KernelSet const *s_kernel;
#endif

static bool is_pow2(UINTa N)
{
    return N != 0 && (N & (N - 1)) == 0;
}

static rfft_complex const *get_dct_twiddle(UINTa N)
{
#ifdef NO_DCT_TABLES
    return NULL;
#else
    if (N*4 <= kMaxN)
        return s_twiddles + N;
    else // DCT twiddles: two more levels! (but only one eight of a circle so the density is different)
        return s_dct_twiddles + ((N/2) - (kMaxN/4));
#endif
}

#ifndef USETABLES

static void calc_twiddle(rfft_complex *twiddle, UINTa count, UINTa freq)
{
    F64 const kPi = 3.1415926535897932384626433832795;
    F64 step = -2.0 * kPi / (F64)freq;
    for (UINTa k = 0; k < count; k++)
    {
        F64 phase = step * (F64)k;
        twiddle[k].re = (F32)cos(phase);
        twiddle[k].im = (F32)sin(phase);
    }
}

static void init_permute_rec(U16 perm[], UINTa mask, UINTa N, UINTa offs_in, UINTa offs_out, UINTa stride)
{
    if (N <= kLeafN)
        perm[offs_in & mask] = (U16) offs_out;
    else
    {
        init_permute_rec(perm, mask, N/2, offs_in, offs_out, stride * 2);
        init_permute_rec(perm, mask, N/4, offs_in + stride, offs_out + N/2, stride * 4);
        if (N/4 >= kLeafN)
            init_permute_rec(perm, mask, N/4, offs_in - stride, offs_out + 3*N/4, stride * 4);
    }
}

static void init_tables()
{
    // Build twiddles
    for (UINTa N = 1; N <= kMaxN / 4; N *= 2)
        calc_twiddle(s_twiddles + N, N, N * 4);

    // Two more at higher freq for DCT
    calc_twiddle(s_dct_twiddles, kMaxN / 4, kMaxN * 2);
    calc_twiddle(s_dct_twiddles + kMaxN / 4, kMaxN / 2, kMaxN * 4);

    // Base permutation table
    init_permute_rec(s_permute + (kMaxN / kLeafN), (kMaxN / kLeafN) - 1, kMaxN, 0, 0, 1);

    // Then subsample to get smaller versions
    for (UINTa N = (kMaxN / kLeafN) / 2; N >= 1; N /= 2)
    {
        U16 const *in = s_permute + N*2;
        U16 *out = s_permute + N;
        for (UINTa i = 0; i < N; i++)
            out[i] = in[i*2];
    }
}

#endif

static int radfft_init_helper()
{
#ifdef CHOOSE_KERNELS
    s_kernel = CHOOSE_KERNELS;
#endif

#ifndef USETABLES
    init_tables();
#endif
    return 1;
}

void RADLINK radfft_init()
{
    // lean on c++ static init here: this is guaranteed by c++ to be called once
    // and will be wrapped with a mutex by the compiler.
    static int done_init = radfft_init_helper();
}

// Complex FFT
void RADLINK radfft_cfft(rfft_complex out[], rfft_complex const in[], UINTa N)
{
    FFTASSERT(is_pow2(N) && N <= kMaxN);

#ifdef WANT_TINY
    if (N <= 2)
    {
        scalar_cfft_tiny(out, in, N);
        return;
    }
#endif

    fft_base_driver(out, in, N, false, s_kernel->radix4, s_kernel->radix2_2);
    if (N >= 8)
        fft_driver(out, N, s_kernel->cfpass);
}

// Complex IFFT
void RADLINK radfft_cifft(rfft_complex out[], rfft_complex const in[], UINTa N)
{
    FFTASSERT(is_pow2(N) && N <= kMaxN);

#ifdef WANT_TINY
    if (N <= 2)
    {
        scalar_cfft_tiny(out, in, N);
        return;
    }
#endif

    fft_base_driver(out, in, N, true, s_kernel->radix4, s_kernel->radix2_2);
    if (N >= 8)
        fft_driver(out, N, s_kernel->cipass);
}

// Real FFT
void RADLINK radfft_rfft(rfft_complex out[], F32 const in[], UINTa N)
{
    FFTASSERT(is_pow2(N) && 2 <= N && N <= kMaxN);
    UINTa N4 = N/4;

    // First do a size-N/2 complex IFFT on "in", computing two
    // size-N/2 FFTs of the even and odd samples from "in".
    radfft_cifft(out, (rfft_complex const *)in, N / 2);

    // We now have FFT_N/2[even_samples] + i*FFT_N/2[odd_samples],
    // and because of symmetries in FFTs of real samples, we can
    // disentangle this just fine.
    //
    // That means we're almost done with a size-N FFT: all we need
    // to do is a final radix-2 butterfly step. "rfpost" does just
    // that: the disentangling using symmetry followed by a final
    // radix-2 butterfly.
    //
    // Again, this is a real FFT which has conjugate symmetry:
    //   x[0..N-1] = input signal
    //   X[0..N-1] = output signal = FFT(x)
    // then X[N - k] = conj(X[k]) (addressing mod N). In particular,
    // this means that X[0] and X[N/2] are real, and the remaining
    // values are uniquely determined by X[1..N/2-1].
    //
    // So we pack the real values for X[0] and X[N/2] into a single
    // complex value at offset 0, and otherwise just return the first
    // half.

    // 0 / Nyquist bins
    rfft_complex v = out[0];
    out[0].re = v.re + v.im;
    out[0].im = v.re - v.im;

    // Remaining bins
    if (N4 > 0)
        s_kernel->rfpost(out, N4, N4);
}

// DCT-II
void RADLINK radfft_dct(F32 out[], F32 in[], UINTa N)
{
    FFTASSERT(is_pow2(N) && 2 <= N && N <= kMaxN);

    // A size-N DCT-II can be expressed as the first N elements of
    // a real DFT of 4N samples
    // [0,in_0,     0,in_1,     0,in_2,     ..., 0,in_{N-1},
    //  0,in_{N-1}, 0,in_{N-2}, 0,in_{N-3}, ..., 0,in_0]
    //
    // First, do a radix-4 Cooley-Tukey DIT step, yielding four
    // size-N sub-DFTs on the expanded input samples. Both of
    // the even sub-DFTs are of all-0 samples (so themselves 0),
    // and the two odd DFTs are closely related because the
    // input vectors are symmetric.
    //
    // Long story short, this reduces into a size-N DFT of
    // permuted input data followed by modulation (point-wise
    // complex multiplication) with a bunch of twiddle factors.

    s_kernel->dct_split(out, in, N);
    radfft_rfft((rfft_complex *)in, out, N);
    s_kernel->dct2_mod(out, in, N, N/2, get_dct_twiddle(N));
}


// Real IFFT
void RADLINK radfft_rifft(F32 out[], rfft_complex in[], UINTa N)
{
    FFTASSERT(is_pow2(N) && 2 <= N && N <= kMaxN);
    UINTa N4 = N/4;

    // Dual to the process in "rfft": we have the first half of the
    // FFT of a signal we know is real (and hence has conjugate
    // symmetry) and want to do the IFFT. So we unpack the packed
    // 0/Nyquist bin and then do a combined symmetric expand and
    // radix-2 butterfly, after which we're back where we were
    // after "radfft_cifft" above:
    //   FFT_N/2[even_samples] + i*FFT_N/2[odd_samples]
    // From the on, it's just a complex IFFT.

    // Pre-pass
    rfft_complex v = in[0];
    in[0].re = 0.5f * (v.re + v.im);
    in[0].im = 0.5f * (v.re - v.im);

    if (N4 > 0)
        s_kernel->ripre(in, N4, N4);

    // Complex IFFT computes the result
    radfft_cfft((rfft_complex *)out, in, N/2);
}

// DCT-III (IDCT)
void RADLINK radfft_idct(F32 out[], F32 in[], UINTa N)
{
    FFTASSERT(is_pow2(N) && 2 <= N && N <= kMaxN);

    // The corresponding IDCT is the inverse of the above:
    // (De)modulate, IFFT, then un-permute.

    s_kernel->dct3_mod(out, in, N, N/2, get_dct_twiddle(N));
    radfft_rifft(in, (rfft_complex *)out, N);
    s_kernel->dct_merge(out, in, N);
}

// DCT-III (IDCT)
void RADLINK radfft_idct_to_S16(S16 outs16[], F32 scale, F32 tmp[], F32 in[], UINTa N)
{
    FFTASSERT(is_pow2(N) && 2 <= N && N <= kMaxN);

    // The corresponding IDCT is the inverse of the above:
    // (De)modulate, IFFT, then un-permute.

    s_kernel->dct3_mod(tmp, in, N, N/2, get_dct_twiddle(N));
    radfft_rifft(in, (rfft_complex *)tmp, N);
    s_kernel->dct_merge_s16(outs16, scale, in, N);
}

// DCT-III (IDCT)
void RADLINK radfft_idct_to_S16_stereo_interleave(S16 outs16[], S16 left[], F32 scale, F32 tmp[], F32 in[], UINTa N)
{
    FFTASSERT(is_pow2(N) && 2 <= N && N <= kMaxN);

    // The corresponding IDCT is the inverse of the above:
    // (De)modulate, IFFT, then un-permute.

    s_kernel->dct3_mod(tmp, in, N, N/2, get_dct_twiddle(N));
    radfft_rifft(in, (rfft_complex *)tmp, N);
    s_kernel->dct_merge_s16s(outs16, left, scale, in, N);
}

#ifdef TABLEGEN

#include <stdio.h>

#define COUNTOF(x) (sizeof(x)/sizeof(*(x)))

static void print_twiddles(FILE *f, char const *name, rfft_complex *vals, UINTa count)
{
    static UINTa const kValsPerLine = 16;
    U32 const *data = (U32 const *)vals;

    count *= 2; // complex numbers are pairs of real values

    fprintf(f, "#define %s ((rfft_complex const *)radfft_%s_data)\n", name, name);
    fprintf(f, "FFTTABLE(U32, radfft_%s_data[%d]) = {\n", name, (int)count);
    for (UINTa i = 0; i < count; ++i)
    {
        if ((i % kValsPerLine) == 0)
            fprintf(f, "  ");

        fprintf(f, "0x%08x,", data[i]);

        if (i == count - 1 || ((i % kValsPerLine) == kValsPerLine - 1))
            fprintf(f, "\n");
    }
    fprintf(f, "};\n\n");
}

static void print_permute(FILE *f, char const *name, Index const *vals, UINTa count)
{
    static UINTa const kValsPerLine = 16;

    fprintf(f, "#define %s ((Index const *)radfft_%s)\n", name, name);
    fprintf(f, "FFTTABLE(Index, radfft_%s[%d]) = {\n", name, (int)count);
    for (UINTa i = 0; i < count; ++i)
    {
        if ((i % kValsPerLine) == 0)
            fprintf(f, "  ");

        fprintf(f, "%4d,", (int)vals[i]);
        if (i == count - 1 || ((i % kValsPerLine) == kValsPerLine - 1))
            fprintf(f, "\n");
        else
            fprintf(f, " ");
    }
    fprintf(f, "};\n\n");
}

int main()
{
    radfft_init();

    static char const filename[] = "radfft_tables.inl";
    FILE *f = fopen(filename, "w");
    if (!f)
    {
        printf("error opening '%s' for writing!\n", filename);
        return 0;
    }

    // Size asserts
    fprintf(f, "typedef int fft_assert_MaxN[(kMaxN == %d) ? 1 : -1];\n", (int)kMaxN);
    fprintf(f, "typedef int fft_assert_LeafN[(kLeafN == %d) ? 1 : -1];\n", (int)kLeafN);
    fprintf(f, "\n");

    print_twiddles(f, "s_twiddles", s_twiddles, COUNTOF(s_twiddles));
    print_twiddles(f, "s_dct_twiddles", s_dct_twiddles, COUNTOF(s_dct_twiddles));
    print_permute(f, "s_permute", s_permute, COUNTOF(s_permute));

    fclose(f);
    printf("%s written.\n", filename);
    return 0;
}

#endif

// vim:et:sts=4:sw=4

