// Copyright Epic Games, Inc. All Rights Reserved.
#ifndef RADFFT_H
#define RADFFT_H

#include "egttypes.h"

#ifdef __RADCONSOLE__
#define RADFFT_ALIGN 32
#define RADFFT_AVX // We know it's always there, might as well use it!
#else
#define RADFFT_ALIGN 16
#endif

#ifdef WRAP_PUBLICS
#define rfmerge3(name,add) name##add
#define rfmerge2(name,add) rfmerge3(name,add)
#define rfmerge(name)      rfmerge2(name,WRAP_PUBLICS)
#define radfft_init                          rfmerge(radfft_init)
#define radfft_cfft                          rfmerge(radfft_cfft)
#define radfft_cifft                         rfmerge(radfft_cifft)
#define radfft_rfft                          rfmerge(radfft_rfft)
#define radfft_rifft                         rfmerge(radfft_rifft)
#define radfft_dct                           rfmerge(radfft_dct)
#define radfft_idct                          rfmerge(radfft_idct)
#define radfft_idct_to_S16                   rfmerge(radfft_idct_to_S16)
#define radfft_idct_to_S16_stereo_interleave rfmerge(radfft_idct_to_S16_stereo_interleave)
#endif


// Complex numbers are returned in this form
typedef struct rfft_complex
{
    F32 re; // real part
    F32 im; // imaginary part
} rfft_complex;

// Initialize. Call this first!
RADDEFFUNC void RADLINK radfft_init();

// Complex FFT. Computes
//   out[k] = sum_{j=0}^{N-1} in[k] * exp(-i*2*pi/N * (j*k))
//
// This is the typical FFT definition (negative sign in the exponential).
//
// N must be a power of 2, "in" and "out" must be aligned to
// RADFFT_ALIGN.
RADDEFFUNC void RADLINK radfft_cfft(rfft_complex out[], rfft_complex const in[], UINTa N);

// Complex IFFT. Computes
//   out[k] = sum_{j=0}^{N-1} in[k] * exp(i*2*pi/N * (j*k))
//
// This is the typical IFFT definition (positive sign in the exponential).
//
//   cifft(fft(x)) = x * N
//
// N must be a power of 2, "in" and "out" must be aligned to
// RADFFT_ALIGN.
RADDEFFUNC void RADLINK radfft_cifft(rfft_complex out[], rfft_complex const in[], UINTa N);

// Real FFT of N elements in[0..N-1].
// This computes (except for out[0] and out[N/2], see below!)
//   out[k] = sum_{j=0}^{N-1} in[j] * exp(i*2*pi/N * (j*k))
//
// Engineers often use the opposite sign convention (-i instead of i in the
// "exp"), and so does the "cfft" in here (sorry). We use this convention
// here because the code this replaced used it.
//
// Returns the first half of the N complex FFT coeffs in out[0..N/2-1].
// The DC bin out[0] is always real for a real FFT.
// We use the imaginary part of out[0] to hold the (also always real) Nyquist bin
// that would otherwise be in out[N/2].
//
// N must be a power of 2, "in" and "out" must be aligned to
// RADFFT_ALIGN.
RADDEFFUNC void RADLINK radfft_rfft(rfft_complex out[], F32 const in[], UINTa N);

// Real IFFT matching "radfft_rfft".
// This computes
//   out[k] = (1/2) * sum_{j=0}^{N-1} in'[j] * exp(-i*2*pi/N * (j*k))
// where
//   in'[0]   = real(in[0])
//   in'[N/2] = imag(in[0])
//   in'[k]   = in[k],         1 <= k < N/2
//   in'[N-k] = conj(in[k]),   1 <= k < N/2
//
// i.e. "in'" is "in" expanded from N/2 to N values with conjugate (Hermitian)
// symmetry (whew). Same comment as for "rfft" applies: this is the opposite
// exponent sign convention from what's typical in engineering, but it matches
// the code we're replacing.
//
// With these rules, we have
//   rifft(rfft(x)) == (N/2) * x
//
// N must be a power of 2, "in" and "out" must be aligned to
// RADFFT_ALIGN.
RADDEFFUNC void RADLINK radfft_rifft(F32 out[], rfft_complex in[], UINTa N);

// DCT-II
//
// Computes the type-II discrete cosine transform ("the" DCT) of "in":
//   out[k] = sum_{j=0}^{N-1} in[j] * cos(pi/N * (j+0.5)*k)
//
// Input data is in in[0..N-1]; output is written to out[0..N-1].
// The input data is destroyed.
//
// N must be a power of 2, "in" and "out" must be aligned to
// RADFFT_ALIGN.
RADDEFFUNC void RADLINK radfft_dct(F32 out[], F32 in[], UINTa N);

// DCT-III
//
// Computes the type-III discrete cosine transform ("the" inverse DCT) of "in":
//   out[k] = sum_{j=0}^{N-1} in[j] * cos(pi/N * j*(k+0.5))
//
// Input data is in in[0..N-1]; output is written to out[0..N-1].
// The input data is destroyed.
//
// Note that this is using the standard normalization which means it's only
// *almost* an inverse of "dct"; to be precise, the inverse of
//
//   dct(data, work, N)
//
// is
//
//   data[0] *= 0.5f;
//   idct(data, work, N);
//   // scale data by 2/N
//
// N must be a power of 2, "in" and "out" must be aligned to
// RADFFT_ALIGN.
RADDEFFUNC void RADLINK radfft_idct(F32 out[], F32 in[], UINTa N);
RADDEFFUNC void RADLINK radfft_idct_to_S16(S16 outs16[], F32 scale, F32 tmp[], F32 in[], UINTa N);
RADDEFFUNC void RADLINK radfft_idct_to_S16_stereo_interleave(S16 outs16[], S16 left[], F32 scale, F32 tmp[], F32 in[], UINTa N);

#endif
