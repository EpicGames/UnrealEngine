// Copyright Epic Games, Inc. All Rights Reserved.
// We really don't use much from SSE3, and what we use is actually
// reasonably easy to synthesize using at most two SSE2 instructions.
// So the idea is to put that code in here and then #include it twice,
// once for real SSE3, once for SSE2 with a few #defines. See radfft.cpp
// for how that's accomplished.

// SSE3 real FFT post-pass.
static void RADFFT_SSE3_PREFIX(rfpost)(rfft_complex out[], UINTa kEnd, UINTa N4)
{
    if (N4 < 2)
    {
        scalar_rfpost(out, N4, N4);
        return;
    }

    F32 const *twiddle = (F32 const *) (s_twiddles + (N4 + 2));
    F32 *out0 = (F32 *) (out + 2);
    F32 *out1 = (F32 *) (out + (N4 * 2) - 3);
    F32 *out0_end = (F32 *) (out + kEnd);

    __m128 conjflip = _mm_setr_ps(0.0f, -0.0f, 0.0f, -0.0f);
    __m128 half = _mm_setr_ps(0.5f, -0.5f, 0.5f, -0.5f);

    // Handle first pair of bins scalar.
    scalar_rfpost(out, 2, N4);

    while (out0 < out0_end)
    {
        __m128 a    = _mm_load_ps(out0);
#if !defined(_MSC_VER) || _MSC_VER != 1600 || defined(__RAD64__) // VC2010 32bit fails to compile this correctly.
        __m128 b    = _mm_loadl_pi(_mm_load_ps(out1 - 2), (__m64 const *) (out1 + 2)); // ==shuffle(loadu(out1), zwxy)
#else
        // this does the same thing but doesn't generate bad code on
        // VC++ 2010 SP1 32-bit
        __m128 b    = _mm_loadu_ps(out1);
        b = _mm_shuffle_ps(b, b, 0x4e);
#endif
        __m128 w    = _mm_load_ps(twiddle);
        __m128 w_re = _mm_moveldup_ps(w);
        __m128 w_im = _mm_movehdup_ps(w);

        // Half-scaled butterfly
        __m128 d    = _mm_mul_ps(_mm_addsub_ps(b, a), half);
        __m128 e    = _mm_add_ps(a, d);

        // Twiddle to get -conj(o) (easier than getting o with what we have)
        __m128 noconj = _mm_addsub_ps(_mm_mul_ps(_mm_shuffle_ps(d, d, 0xb1), w_re), _mm_mul_ps(d, w_im));

        // Generate outputs
        __m128 o0    = _mm_addsub_ps(e, noconj); // e + o
        __m128 o1    = _mm_add_ps(_mm_xor_ps(e, conjflip), noconj);

        // Store
        _mm_store_ps(out0, o0);
        _mm_storeu_ps(out1, _mm_shuffle_ps(o1, o1, 0x4e));

        out0 += 4;
        out1 -= 4;
        twiddle += 4;
    }
}

// SSE3 real IFFT pre-pass.
static void RADFFT_SSE3_PREFIX(ripre)(rfft_complex out[], UINTa kEnd, UINTa N4)
{
    if (N4 < 2)
    {
        scalar_rfpost(out, N4, N4);
        return;
    }

    F32 const *twiddle = (F32 const *) (s_twiddles + (N4 + 2));
    F32 *out0 = (F32 *) (out + 2);
    F32 *out1 = (F32 *) (out + (N4 * 2) - 3);
    F32 *out0_end = (F32 *) (out + kEnd);

    __m128 conjflip = _mm_setr_ps(0.0f, -0.0f, 0.0f, -0.0f);
    __m128 half = _mm_setr_ps(-0.5f, 0.5f, -0.5f, 0.5f);

    // Handle first pair of bins scalar
    scalar_ripre(out, 2, N4);

    while (out0 < out0_end)
    {
        __m128 a    = _mm_load_ps(out0);
#if !defined(_MSC_VER) || _MSC_VER != 1600 || defined(__RAD64__) // VC2010 32bit fails to compile this correctly.
        __m128 b    = _mm_loadl_pi(_mm_load_ps(out1 - 2), (__m64 const *) (out1 + 2)); // ==shuffle(loadu(out1), zwxy)
#else
        // this does the same thing but doesn't generate bad code on
        // VC++ 2010 SP1 32-bit
        __m128 b    = _mm_loadu_ps(out1);
        b = _mm_shuffle_ps(b, b, 0x4e);
#endif
        __m128 w    = _mm_load_ps(twiddle);
        __m128 w_re = _mm_moveldup_ps(w);
        __m128 w_im = _mm_movehdup_ps(w);

        w_im = _mm_xor_ps(w_im, conjflip);

        // Half-scaled butterfly
        __m128 d    = _mm_mul_ps(_mm_addsub_ps(b, a), half);
        __m128 e    = _mm_sub_ps(a, d);

        // Twiddle to get -conj(o) (easier than getting o with what we have)
        __m128 noconj = _mm_add_ps(_mm_mul_ps(_mm_shuffle_ps(d, d, 0xb1), w_re), _mm_mul_ps(d, w_im));

        // Generate outputs
        __m128 o0    = _mm_addsub_ps(e, noconj); // e + o
        __m128 o1    = _mm_add_ps(_mm_xor_ps(e, conjflip), noconj);

        // Store
        _mm_store_ps(out0, o0);
        _mm_storeu_ps(out1, _mm_shuffle_ps(o1, o1, 0x4e));

        out0 += 4;
        out1 -= 4;
        twiddle += 4;
    }
}

// SSE3 complex forward conjugate split-radix reduction pass. This is the main workhorse inner loop for FFTs.
static void RADFFT_SSE3_PREFIX(cfpass)(rfft_complex data[], PlanElement const *plan, UINTa Nover4)
{
    __m128 conjflip = _mm_setr_ps(0.0f, -0.0f, 0.0f, -0.0f);
    --plan;

    do
    {
        ++plan;
        UINTa N1 = plan->Nloop;
        UINTa step = N1 * 2;
        F32 *out = (F32 *) (data + plan->offs);
        F32 *out_end = out + step;

        F32 const *twiddle = (F32 const *) (s_twiddles + N1);
        do
        {
            __m128 Zk       = _mm_load_ps(out + 2*step);
            __m128 Zpk      = _mm_load_ps(out + 3*step);
            __m128 w        = _mm_load_ps(twiddle);
            __m128 w_re     = _mm_moveldup_ps(w);
            __m128 w_im     = _mm_movehdup_ps(w);

            // Twiddle Zk, Z'k
            Zk  = _mm_addsub_ps(_mm_mul_ps(Zk, w_re), _mm_mul_ps(_mm_shuffle_ps(Zk, Zk, 0xb1 /* yxwz */), w_im));
            Zpk = _mm_addsub_ps(_mm_mul_ps(_mm_shuffle_ps(Zpk, Zpk, 0xb1), w_re), _mm_mul_ps(Zpk, w_im));

            __m128 Zsum = _mm_add_ps(_mm_shuffle_ps(Zpk, Zpk, 0xb1), Zk);
            __m128 Zdif = _mm_sub_ps(_mm_shuffle_ps(Zk, Zk, 0xb1), Zpk);

            // Even inputs
            __m128 Uk0      = _mm_load_ps(out + 0*step);
            __m128 Uk1      = _mm_load_ps(out + 1*step);

            // Output butterflies
            _mm_store_ps(out + 0*step, _mm_add_ps(Uk0,    Zsum));
            _mm_store_ps(out + 1*step, _mm_add_ps(Uk1,    _mm_xor_ps(Zdif, conjflip)));
            _mm_store_ps(out + 2*step, _mm_sub_ps(Uk0,    Zsum));
            _mm_store_ps(out + 3*step, _mm_addsub_ps(Uk1, Zdif));

            out += 4;
            twiddle += 4;
        } while (out < out_end);
    } while (plan->Nloop < Nover4);
}

// SSE3 complex inverse conjugate split-radix reduction pass. This is the main workhorse inner loop for IFFTs.
static void RADFFT_SSE3_PREFIX(cipass)(rfft_complex data[], PlanElement const *plan, UINTa Nover4)
{
    __m128 conjflip = _mm_setr_ps(0.0f, -0.0f, 0.0f, -0.0f);
    --plan;

    do
    {
        ++plan;
        UINTa N1 = plan->Nloop;
        UINTa step = N1 * 2;
        F32 *out = (F32 *) (data + plan->offs);
        F32 *out_end = out + step;

        F32 const *twiddle = (F32 const *) (s_twiddles + N1);
        do
        {
            __m128 Zk       = _mm_load_ps(out + 2*step);
            __m128 Zpk      = _mm_load_ps(out + 3*step);
            __m128 w        = _mm_load_ps(twiddle);
            __m128 w_re     = _mm_moveldup_ps(w);
            __m128 w_im     = _mm_movehdup_ps(w);

            // Twiddle Zk, Z'k
            Zpk = _mm_addsub_ps(_mm_mul_ps(Zpk, w_re), _mm_mul_ps(_mm_shuffle_ps(Zpk, Zpk, 0xb1), w_im));
            Zk  = _mm_addsub_ps(_mm_mul_ps(_mm_shuffle_ps(Zk, Zk, 0xb1), w_re), _mm_mul_ps(Zk, w_im));

            __m128 Zsum = _mm_add_ps(_mm_shuffle_ps(Zk, Zk, 0xb1), Zpk);
            __m128 Zdif = _mm_sub_ps(_mm_shuffle_ps(Zpk, Zpk, 0xb1), Zk);

            // Even inputs
            __m128 Uk0      = _mm_load_ps(out + 0*step);
            __m128 Uk1      = _mm_load_ps(out + 1*step);

            // Output butterflies
            _mm_store_ps(out + 0*step, _mm_add_ps(Uk0,    Zsum));
            _mm_store_ps(out + 1*step, _mm_add_ps(Uk1,    _mm_xor_ps(Zdif, conjflip)));
            _mm_store_ps(out + 2*step, _mm_sub_ps(Uk0,    Zsum));
            _mm_store_ps(out + 3*step, _mm_addsub_ps(Uk1, Zdif));

            out += 4;
            twiddle += 4;
        } while (out < out_end);
    } while (plan->Nloop < Nover4);
}

static void RADFFT_SSE3_PREFIX(dct2_modulate)(F32 out[], F32 const in[], UINTa N, UINTa Nlast, rfft_complex const *twiddle)
{
    if (N < 8)
    {
        scalar_dct2_modulate(out, in, N, Nlast, twiddle);
        return;
    }

    // First few bins have exceptional cases, let scalar routine handle it
    FFTASSERT((Nlast % 4) == 0 && Nlast >= 4);
    scalar_dct2_modulate(out, in, N, 4, twiddle);

    for (UINTa k = 4; k < Nlast; k += 2)
    {
        __m128 v    = _mm_load_ps(in + k*2);
        __m128 w    = _mm_load_ps(&twiddle[k].re);
        __m128 w_re = _mm_moveldup_ps(w);
        __m128 w_im = _mm_movehdup_ps(w);

        __m128 x    = _mm_addsub_ps(_mm_mul_ps(_mm_shuffle_ps(v, v, 0xb1), w_re), _mm_mul_ps(v, w_im));
        __m128 s    = _mm_shuffle_ps(x, x, 0x2d);

        _mm_storel_pi((__m64 *) (out + k), s);
        _mm_storeh_pi((__m64 *) (out + N - 1 - k), s);
    }
}

// vim:et:sts=4:sw=4

