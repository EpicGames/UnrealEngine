// Copyright Epic Games, Inc. All Rights Reserved.
#ifndef __RADMATHH__
#define __RADMATHH__

#include <math.h>

RADDEFSTART

#define mult64anddiv( a, b, c )  ( (U32) ( ( ( (U64) a ) * ( (U64) b ) ) / ( (U64) c ) ) )
#define mult64andshift( a, b, c )  ( (U32) ( ( ( (U64) a ) * ( (U64) b ) ) >> ( (U64) c ) ) )
#define radabs abs

// We use the floating point version of everything and have casts around everything
// to ensure we don't accidentally have an expression evaluate as double on one platform
// and float on the other.
//
// This was due to previously using e.g. sqrt on one platform and sqrtf on another, but there's
// no reason to _not_ have these casts.
#define radatan2( val1, val2 )    ((float)atan2f  ( (float)val1, (float)val2) )
#define radpow( val1, val2 )      ((float)powf    ( (float)val1, (float)val2) )
#define radfsqrt( val )           ((float)sqrtf   ( (float)val) )
#define radlog10( val )           ((float)log10f  ( (float)val) )
#define radexp( val )             ((float)expf    ( (float)val) )
#define radfabs( val )            ((float)fabsf   ( (float)val) )
#define radfloor( val )           ((float)floorf  ( (float)val) )

float ranged_log_0p05_to_0p5(float x);

#if !defined( _MSC_VER )
  #include <stdlib.h> // abs
#endif

RADDEFEND

#endif
