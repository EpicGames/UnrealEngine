// Copyright Epic Games, Inc. All Rights Reserved.
//
// In order to avoid potential consistency issues between libms we have this narrow-range log function
// as we only use it in one place inside binkace and we only use it within a small range.
//
// 
// handles 0.05 to 0.5 (can easily be full range with an if and no table for the ln2*scale)
extern "C" float ranged_log_0p05_to_0p5(float x) 
{
	static float twotoe[5] = { 4.0f*0.69314718f, 3.0f*0.69314718f, 2.0f*0.69314718f, 1.0f*0.69314718f, 0.0f };  // handles range of 0.05 to 0.5 only needs 5 entryies (final value zero is just for 0.5)
	static int k5 = 0x3fdef298; // close to -1.7417939f, but scanned to minimize max error (which is 0.000061170932 with ave of 0.00003781923)
	union { int i; float f; } f;
	f.f = x;
	f.i &= 0xff800000; // isolates the mantissa and sign
	x = f.f / x;  // gets a less-than-1.0 value into range of 0.5 to 1
	return *((float*)&k5) + ( -5.64234426f + ( 5.87976061f + ( -3.57742942f + 0.905148439f * x) * x) * x) * x - twotoe[ (f.i>>23) - 122 ]; // constants from maple, using remez
}
