// Copyright Epic Games, Inc. All Rights Reserved.
#if !defined(__RADRR_CORE2H__)
#define __RADRR_CORE2H__

#include "egttypes.h"

#include <stddef.h>

#ifdef __RAD64__
  #define RAD_PTRBITS 64
  #define RAD_PTRBYTES 8
  #define RAD_TWOPTRBYTES 16
#else
  #define RAD_PTRBITS 32
  #define RAD_PTRBYTES 4
  #define RAD_TWOPTRBYTES 8
#endif

#if defined(__RADMAC__)
  #define RR_BREAK() __builtin_trap()
  #define RAD_GUARANTEED_SSE3   // always sse3 on mac
  #define RAD_USES_SSE3         // don't need any special markup for sse3 on mac
  #define RAD_USES_SSSE3        // don't need any special markup for ssse3 on mac
#endif
#if defined(__RADIPHONE__)
  #define RR_BREAK() __builtin_trap()
  #define RAD_GUARANTEED_SSE3   // simulator has sse3
  #define RAD_USES_SSE3         // don't need any special markup for sse3 on sim
#endif
#if defined(__RADANDROID__)
  #define RR_BREAK() __builtin_trap()
#endif

#if defined(__RADLINUX__)
  #define RR_BREAK() __builtin_trap()
#endif
#if defined(__RADQNX__)
  #define RR_BREAK() __builtin_trap()
#endif
#if defined(__RADWINRT__)
  #ifdef __RAD64__
    #define __RAD64REGS__
  #endif
  #define RR_BREAK __debugbreak
#endif
#if defined(__RADNT__)
  #ifdef __RAD64__
    #define __RAD64REGS__
  #endif
    #define RR_BREAK __debugbreak
#endif

#if defined(__RADEMSCRIPTEN__)
  #define RR_BREAK() __builtin_trap()
#endif

#if !defined(RR_BREAK) 
  #error "No rr_break defined!"
#endif


#ifdef __RAD64REGS__
  #define RAD_UINTr RAD_U64
  #define RAD_SINTr RAD_S64
#else
  #define RAD_UINTr RAD_U32
  #define RAD_SINTr RAD_S32
#endif

// make sure your symbols are unique with these next macros!
//   if you have a duplicate symbol and they are different sizes,
//   then you will get a warning.  Same size, though?  MERGED!! Careful!
// Use RRSTRIPPABLE on private vars
// Use RRSTRIPPABLEPUB on public vars
#ifdef _MSC_VER
#define RRSTRIPPABLE __declspec(selectany)
#define RRSTRIPPABLEPUB __declspec(selectany)
#else
#define RRSTRIPPABLE static
#define RRSTRIPPABLEPUB 
#endif

//===========================================================================
// RADASSUME(expr) tells the compiler that expr is always true
// RADFORCEINLINE is what you would expect
#ifdef _MSC_VER
  #if (_MSC_VER >= 1300)
    #define RAD_ALIGN(type,var,num) type __declspec(align(num)) var
  #else
  #endif

  #ifndef RAD_USES_SSE3
    #define RAD_USES_SSE3     // vs2008+ can compile SSE3 with no special markup
  #endif
  #ifndef RAD_USES_SSSE3
    #define RAD_USES_SSSE3    // vs2008+ can compile SSSE3 with no special markup
  #endif

  #define RADFORCEINLINE __forceinline
  #define RADASSUME(exp) __assume(exp)

#elif defined(__clang__) //================================

  #define RAD_ALIGN(type,var,num) type __attribute__ ((aligned (num))) var
  #ifdef _DEBUG
    #define RADFORCEINLINE inline
  #else
    #define RADFORCEINLINE inline __attribute((always_inline))
  #endif

  #ifndef RAD_USES_SSE3
    #define RAD_USES_SSE3   __attribute__((target("sse3"))) 
  #endif
  #if !defined(RAD_USES_SSSE3) && !defined(__RADIPHONE__) // if it isn't already defined, define SSSE3 macro for all platforms except iphone sim!
    #define RAD_USES_SSSE3  __attribute__((target("ssse3")))
  #endif

  #if __has_builtin(__builtin_assume)
    #define RADASSUME(exp) __builtin_assume(exp)
  #else
    #define RADASSUME(exp)  RAD_STATEMENT_WRAPPER( if ( ! (exp) ) __builtin_unreachable(); )
  #endif

#elif (defined(__GCC__) || defined(__GNUC__)) || defined(ANDROID) //================================

  #define RAD_ALIGN(type,var,num) type __attribute__ ((aligned (num))) var
  #ifdef _DEBUG
    #define RADFORCEINLINE inline
  #else
    #define RADFORCEINLINE inline __attribute((always_inline))
  #endif

  #define __RAD_GCC_VERSION__ (__GNUC__ * 10000 \
                             + __GNUC_MINOR__ * 100 \
                             + __GNUC_PATCHLEVEL__)

  #if __RAD_GCC_VERSION__ >= 40500
    #define RADASSUME(exp)  RAD_STATEMENT_WRAPPER( if ( ! (exp) ) __builtin_unreachable(); )
  #else
    #define RADASSUME(exp)
  #endif

  #if __RAD_GCC_VERSION__ >= 40600
    #ifndef RAD_USES_SSE3
      #define RAD_USES_SSE3   __attribute__((target("sse3"))) 
    #endif
    #ifndef RAD_USES_SSSE3
      #define RAD_USES_SSSE3  __attribute__((target("ssse3")))
    #endif
  #else
    // on ancient GCC, turn off SSSE3
    #ifdef RAD_USES_SSSE3
      #undef RAD_USES_SSSE3
    #endif
  #endif

#else
  #define RADASSUME(exp)       !! no RADASSUME defined      !!
  #define RADFORCEINLINE       !! no RADFORCEINLINE defined !!
#endif    

//===========================================================================
// RAD_STATEMENT_START AND RAD_STATEMENT_END_FALSE and END_TRUE
//   are used for "do{}while(0) or do{}while(1) loops without
//   warnings hopefully.
// RAD_STATEMENT_WRAPPER(code) is for doing the tart/end_false all at once
// RAD_INFINITE_LOOP(code) is for doing the tart/end_true all at once
#ifdef _MSC_VER
  // microsoft compilers
  #if _MSC_VER >= 1400
    #define RAD_STATEMENT_START \
      do {

    #define RAD_STATEMENT_END_FALSE \
       __pragma(warning(push)) \
      __pragma(warning(disable:4127)) \
    } while(0) \
    __pragma(warning(pop)) 

    #define RAD_STATEMENT_END_TRUE \
       __pragma(warning(push)) \
      __pragma(warning(disable:4127)) \
    } while(1) \
    __pragma(warning(pop))

  #else
    #define RAD_USE_STANDARD_LOOP_CONSTRUCT  
  #endif
#else
  #define RAD_USE_STANDARD_LOOP_CONSTRUCT  
#endif

#ifdef RAD_USE_STANDARD_LOOP_CONSTRUCT
  #define RAD_STATEMENT_START \
    do {

  #define RAD_STATEMENT_END_FALSE \
    } while ( (void)0,0 )
    
  #define RAD_STATEMENT_END_TRUE \
    } while ( (void)1,1 )
#endif

#define RAD_STATEMENT_WRAPPER( code ) \
  RAD_STATEMENT_START \
    code \
  RAD_STATEMENT_END_FALSE
  
#define RAD_INFINITE_LOOP( code ) \
  RAD_STATEMENT_START \
    code \
  RAD_STATEMENT_END_TRUE


//===========================================================================
// to kill unused var warnings.
// Must be placed after variable declarations for code compiled as .c
#if defined(_MSC_VER) && _MSC_VER >= 1600 // in 2010 aka 10.0 and later 
  #define RR_UNUSED_VARIABLE(x) (void) (x)
#else
  #define RR_UNUSED_VARIABLE(x) (void)(sizeof(x))
#endif

//===========================================================================
// RR_LINESTRING is the current line number as a string
#define RR_STRINGIZE( L )         #L
#define RR_DO_MACRO( M, X )       M(X)
#define RR_STRINGIZE_DELAY( X )   RR_DO_MACRO( RR_STRINGIZE, X )
#define RR_LINESTRING             RR_STRINGIZE_DELAY( __LINE__ )

#define RR_CAT(X,Y)                 X ## Y

#define RR_STRING_JOIN3(arg1, arg2, arg3)            RR_STRING_JOIN_DELAY3(arg1, arg2, arg3)
#define RR_STRING_JOIN_DELAY3(arg1, arg2, arg3)      RR_STRING_JOIN_IMMEDIATE3(arg1, arg2, arg3)
#define RR_STRING_JOIN_IMMEDIATE3(arg1, arg2, arg3)  arg1 ## arg2 ## arg3

//===========================================================================
// Range macros

#ifndef RR_MIN
  #define RR_MIN(a,b)    ( (a) < (b) ? (a) : (b) )
#endif

#ifndef RR_MAX
  #define RR_MAX(a,b)    ( (a) > (b) ? (a) : (b) )
#endif

#ifndef RR_ABS
  #define RR_ABS(a)      ( ((a) < 0) ? -(a) : (a) )
#endif

#ifndef RR_CLAMP
  #define RR_CLAMP(val,lo,hi) RR_MAX( RR_MIN(val,hi), lo )
#endif

//===========================================================================
// Data layout macros

#define RR_ARRAY_SIZE(array)  ( sizeof(array)/sizeof(array[0]) )

// MEMBER_OFFSET tells you the offset of a member in a type
#define RR_MEMBER_OFFSET(type,member) offsetof(type, member)

// MEMBER_SIZE tells you the size of a member in a type
#define RR_MEMBER_SIZE(type,member)  ( sizeof( ((type *) 0)->member) )

// just to make gcc shut up about derefing null :
#define RR_MEMBER_OFFSET_PTR(type,member,ptr)  ( (SINTa) &(((type *)(ptr))->member)  - (SINTa)(ptr) )
#define RR_MEMBER_SIZE_PTR(type,member,ptr)	( sizeof( ((type *) (ptr))->member) )
 
// MEMBER_TO_OWNER takes a pointer to a member and gives you back the base of the object
//  you should then RR_ASSERT( &(ret->member) == ptr );
#define RR_MEMBER_TO_OWNER(type,member,ptr)    (type *)( ((char *)(ptr)) - RR_MEMBER_OFFSET_PTR(type,member,ptr) )


//===========================================================================
// Cache / prefetch macros :
// RR_PREFETCH for various platforms :
// 
// RR_PREFETCH_SEQUENTIAL : prefetch memory for reading in a sequential scan
//		platforms that automatically prefetch sequential (eg. PC) should be a no-op here
// RR_PREFETCH_WRITE_INVALIDATE : prefetch memory for writing - contents of memory are undefined
//		(may be a no-op, may be a normal prefetch, may zero memory)
//		warning : RR_PREFETCH_WRITE_INVALIDATE may write memory so don't do it past the end of buffers

#ifdef __RADX86__
  #define RR_PREFETCH_SEQUENTIAL(ptr,offset)	// nop
  #define RR_PREFETCH_WRITE_INVALIDATE(ptr,offset)	// nop
#elif defined(__RADANDROID__)
  #define RR_PREFETCH_SEQUENTIAL(ptr,offset)   // intentional NOP for now
  #define RR_PREFETCH_WRITE_INVALIDATE(ptr,offset)  // nop
#elif defined(__RADWINRT__)
  #define RR_PREFETCH_SEQUENTIAL(ptr,offset)   __prefetch((char *)(ptr) + (int)(offset))
  #define RR_PREFETCH_WRITE_INVALIDATE(ptr,offset)  // nop
#elif defined(__RADARM__)
  #define RR_PREFETCH_SEQUENTIAL(ptr,offset)   // intentional NOP for now
  #define RR_PREFETCH_WRITE_INVALIDATE(ptr,offset)  // nop
#else
  // other platforms - intentionally produce a compile error 
  #ifndef RR_PREFETCH_SEQUENTIAL
    #define RR_PREFETCH_SEQUENTIAL(ptr,offset)		    !! NO PREFETCH DEFINED !!
  #endif
  #ifndef RR_PREFETCH_WRITE_INVALIDATE
    #define RR_PREFETCH_WRITE_INVALIDATE(ptr,offset)	 !! NO PREFETCH DEFINED !!
  #endif
#endif

//===========================================================================
// LIGHTWEIGHT ASSERTS without rrAssert.h

//===========================================================================
#if (defined(_DEBUG) && !defined(NDEBUG)) || defined(ASSERT_IN_RELEASE)
  #define RR_DO_ASSERTS
#endif

/*
RR_ASSERT(exp) - the normal assert thing, toggled with RR_DO_ASSERTS
RR_ASSERT_ALWAYS(exp) - assert that you want to test even in ALL builds (including final!)
RR_ASSERT_RELEASE(exp) - assert that you want to test even in release builds (not for final!)
RR_ASSERT_LITE(exp) - normal assert is not safe from threads or inside malloc; use this instead
RR_DURING_ASSERT(exp) - wrap operations that compute stuff for assert in here
RR_DO_ASSERTS - toggle tells you if asserts are enabled or not

RR_BREAK() - generate a debug break - always !
RR_ASSERT_BREAK() - RR_BREAK for asserts ; disable with RAD_NO_BREAK

RR_ASSERT_FAILURE(str)  - just break with a messsage; like assert with no condition
RR_ASSERT_FAILURE_ALWAYS(str)  - RR_ASSERT_FAILURE in release builds too
RR_CANT_GET_HERE() - put in spots execution should never go
RR_COMPILER_ASSERT(exp) - checks constant conditions at compile time

RADTODO - note to search for nonfinal stuff
RR_PRAGMA_MESSAGE - message dealy, use with #pragma in MSVC

*************/

//-----------------------------------
#ifdef RR_DO_ASSERTS 
  #define RR_ASSERT(exp)           RAD_STATEMENT_WRAPPER( if ( ! (exp) ) { RR_BREAK(); } )
#else 
  #define RR_ASSERT(exp)           (void)0
#endif // RR_DO_ASSERTS

#define rrAssert  RR_ASSERT
#define rrassert  RR_ASSERT

#ifdef _MSC_VER
  // without this, our assert errors...
  #if _MSC_VER >= 1300
    #pragma warning( disable : 4127) // conditional expression is constant
  #endif
#endif

//=================================================================
// Get/Put from memory in little or big endian :
//
// val = RR_GET32_BE(ptr)
// RR_PUT32_BE(ptr,val)
//
//  available here :
//		RR_[GET/PUT][16/32]_[BE/LE][_UNALIGNED][_OFFSET]
//
//	if you don't specify _UNALIGNED , then ptr & offset shoud both be aligned to type size
//	_OFFSET is in *bytes* !

// you can #define RR_GET_RESTRICT to make all RR_GETs be RESTRICT
// if you set nothing they are not

#ifdef RR_GET_RESTRICT
  #define RR_GET_PTR_POST RADRESTRICT
#endif
#ifndef RR_GET_PTR_POST
  #define RR_GET_PTR_POST
#endif

// native version of get/put is always trivial :

#define RR_GET16_NATIVE(ptr)     *((const U16 * RR_GET_PTR_POST)(ptr))
#define RR_PUT16_NATIVE(ptr,val) *((U16 * RR_GET_PTR_POST)(ptr)) = (U16)(val)

// offset is in bytes
#define RR_U16_PTR_OFFSET(ptr,offset)          ((U16 * RR_GET_PTR_POST)((char *)(ptr) + (offset)))
#define RR_GET16_NATIVE_OFFSET(ptr,offset)     *( RR_U16_PTR_OFFSET((ptr),offset) )
#define RR_PUT16_NATIVE_OFFSET(ptr,val,offset) *( RR_U16_PTR_OFFSET((ptr),offset)) = (val)

#define RR_GET32_NATIVE(ptr)     *((const U32 * RR_GET_PTR_POST)(ptr))
#define RR_PUT32_NATIVE(ptr,val) *((U32 * RR_GET_PTR_POST)(ptr)) = (val)

// offset is in bytes
#define RR_U32_PTR_OFFSET(ptr,offset)          ((U32 * RR_GET_PTR_POST)((char *)(ptr) + (offset)))
#define RR_GET32_NATIVE_OFFSET(ptr,offset)     *( RR_U32_PTR_OFFSET((ptr),offset) )
#define RR_PUT32_NATIVE_OFFSET(ptr,val,offset) *( RR_U32_PTR_OFFSET((ptr),offset)) = (val)

#define RR_GET64_NATIVE(ptr)     *((const U64 * RR_GET_PTR_POST)(ptr))
#define RR_PUT64_NATIVE(ptr,val) *((U64 * RR_GET_PTR_POST)(ptr)) = (val)

// offset is in bytes
#define RR_U64_PTR_OFFSET(ptr,offset)          ((U64 * RR_GET_PTR_POST)((char *)(ptr) + (offset)))
#define RR_GET64_NATIVE_OFFSET(ptr,offset)     *( RR_U64_PTR_OFFSET((ptr),offset) )
#define RR_PUT64_NATIVE_OFFSET(ptr,val,offset) *( RR_U64_PTR_OFFSET((ptr),offset)) = (val)

//---------------------------------------------------

#ifdef __RADLITTLEENDIAN__
  #define RR_GET16_LE     RR_GET16_NATIVE
  #define RR_PUT16_LE     RR_PUT16_NATIVE
  #define RR_GET16_LE_OFFSET     RR_GET16_NATIVE_OFFSET
  #define RR_PUT16_LE_OFFSET     RR_PUT16_NATIVE_OFFSET

  #define RR_GET32_LE     RR_GET32_NATIVE
  #define RR_PUT32_LE     RR_PUT32_NATIVE
  #define RR_GET32_LE_OFFSET     RR_GET32_NATIVE_OFFSET
  #define RR_PUT32_LE_OFFSET     RR_PUT32_NATIVE_OFFSET

  #define RR_GET64_LE     RR_GET64_NATIVE
  #define RR_PUT64_LE     RR_PUT64_NATIVE
  #define RR_GET64_LE_OFFSET     RR_GET64_NATIVE_OFFSET
  #define RR_PUT64_LE_OFFSET     RR_PUT64_NATIVE_OFFSET
#else
  #define RR_GET16_BE     RR_GET16_NATIVE
  #define RR_PUT16_BE     RR_PUT16_NATIVE
  #define RR_GET16_BE_OFFSET     RR_GET16_NATIVE_OFFSET
  #define RR_PUT16_BE_OFFSET     RR_PUT16_NATIVE_OFFSET

  #define RR_GET32_BE     RR_GET32_NATIVE
  #define RR_PUT32_BE     RR_PUT32_NATIVE
  #define RR_GET32_BE_OFFSET     RR_GET32_NATIVE_OFFSET
  #define RR_PUT32_BE_OFFSET     RR_PUT32_NATIVE_OFFSET

  #define RR_GET64_BE     RR_GET64_NATIVE
  #define RR_PUT64_BE     RR_PUT64_NATIVE
  #define RR_GET64_BE_OFFSET     RR_GET64_NATIVE_OFFSET
  #define RR_PUT64_BE_OFFSET     RR_PUT64_NATIVE_OFFSET
#endif

//-------------------------
// non-native Get/Put implementations go here :

#if defined(__RADX86__)
  // good implementation for X86 :
  #if (_MSC_VER >= 1300)
    RADDEFFUNC unsigned short __cdecl _byteswap_ushort (unsigned short _Short);
    RADDEFFUNC unsigned long  __cdecl _byteswap_ulong  (unsigned long  _Long);
    #pragma intrinsic(_byteswap_ushort, _byteswap_ulong)

    #define RR_BSWAP16   _byteswap_ushort
    #define RR_BSWAP32  _byteswap_ulong

    RADDEFFUNC unsigned __int64 __cdecl _byteswap_uint64 (unsigned __int64 val);
    #pragma intrinsic(_byteswap_uint64)
    #define RR_BSWAP64  _byteswap_uint64
  #elif defined(_MSC_VER) // VC6
    RADDEFFUNC RADFORCEINLINE unsigned long RR_BSWAP16 (unsigned long _Long)
    {
      __asm {
        mov eax, [_Long]
        rol ax, 8
        mov [_Long], eax;
      }
      return _Long;
    }

    RADDEFFUNC RADFORCEINLINE unsigned long RR_BSWAP32 (unsigned long _Long)
    {
      __asm {
        mov eax, [_Long]
        bswap eax
        mov [_Long], eax
      }
      return _Long;
    }

    RADDEFFUNC RADFORCEINLINE unsigned __int64 RR_BSWAP64 (unsigned __int64 _Long)
    {
      __asm {
        mov eax, DWORD PTR _Long
        mov edx, DWORD PTR _Long+4
        bswap eax
        bswap edx
        mov DWORD PTR _Long, edx
        mov DWORD PTR _Long+4, eax
      }
      return _Long;
    }
  #elif defined(__GNUC__) || defined(__clang__)
    // GCC has __builtin_bswap16, but Clang only seems to have added it recently.
    // We use __builtin_bswap32/64 but 16 just uses the macro version. (No big
    // deal if that turns into shifts anyway)
    #define RR_BSWAP16(u16) ( (U16) ( ((u16) >> 8) | ((u16) << 8) ) )
    #define RR_BSWAP32  __builtin_bswap32
    #define RR_BSWAP64  __builtin_bswap64
  #endif

  #define RR_GET16_BE(ptr)        RR_BSWAP16(*((U16 *)(ptr)))
  #define RR_PUT16_BE(ptr,val)    *((U16 *)(ptr)) = (U16) RR_BSWAP16(val)
  #define RR_GET16_BE_OFFSET(ptr,offset)        RR_BSWAP16(*RR_U16_PTR_OFFSET(ptr,offset))
  #define RR_PUT16_BE_OFFSET(ptr,val,offset)    *RR_U16_PTR_OFFSET(ptr,offset) = RR_BSWAP16(val)

  #define RR_GET32_BE(ptr)        RR_BSWAP32(*((U32 *)(ptr)))
  #define RR_PUT32_BE(ptr,val)    *((U32 *)(ptr)) = RR_BSWAP32(val)
  #define RR_GET32_BE_OFFSET(ptr,offset)        RR_BSWAP32(*RR_U32_PTR_OFFSET(ptr,offset))
  #define RR_PUT32_BE_OFFSET(ptr,val,offset)    *RR_U32_PTR_OFFSET(ptr,offset) = RR_BSWAP32(val)

  #define RR_GET64_BE(ptr)        RR_BSWAP64(*((U64 *)(ptr)))
  #define RR_PUT64_BE(ptr,val)    *((U64 *)(ptr)) = RR_BSWAP64(val)
  #define RR_GET64_BE_OFFSET(ptr,offset)        RR_BSWAP64(*RR_U64_PTR_OFFSET(ptr,offset))
  #define RR_PUT64_BE_OFFSET(ptr,val,offset)    *RR_U64_PTR_OFFSET(ptr,offset) = RR_BSWAP64(val)
  // end _MSC_VER
#elif defined(__RADIPHONE__) || defined(__RADANDROID__)
  // iPhone/Android does not seem to have intrinsics for this, so use generic fallback!

  // Bswap is just here for use of implementing get/put
  //  caller should use Get/Put , not bswap
  #define RR_BSWAP16(u16) ( (U16) ( ((u16) >> 8) | ((u16) << 8) ) )
  #define RR_BSWAP32(u32) ( (U32) ( ((u32) >> 24) | (((u32)<<8) & 0x00FF0000) | (((u32)>>8) & 0x0000FF00) | ((u32) << 24) ) )
  #define RR_BSWAP64(u64) ( (U64) ( RR_BSWAP32((u64)>>32) | ((U64)RR_BSWAP32((U32)(u64))<<32) ) )

  #define RR_GET16_BE(ptr)        RR_BSWAP16(*((U16 *)(ptr)))
  #define RR_PUT16_BE(ptr,val)    *((U16 *)(ptr)) = RR_BSWAP16(val)

  #define RR_GET32_BE(ptr)        RR_BSWAP32(*((U32 *)(ptr)))
  #define RR_PUT32_BE(ptr,val)    *((U32 *)(ptr)) = RR_BSWAP32(val)

  #define RR_GET64_BE(ptr)        RR_BSWAP64(*((U64 *)(ptr)))
  #define RR_PUT64_BE(ptr,val)    *((U64 *)(ptr)) = RR_BSWAP64(val)
#elif defined(__RADWINRTAPI__) && defined(__RADARM__)
  #include <intrin.h>
  #define RR_BSWAP16(u16)         _arm_rev16(u16)
  #define RR_BSWAP32(u32)         _arm_rev(u32)

  #define RR_GET16_BE(ptr)        RR_BSWAP16(*((U16 *)(ptr)))
  #define RR_PUT16_BE(ptr,val)    *((U16 *)(ptr)) = RR_BSWAP16(val)

  #define RR_GET32_BE(ptr)        RR_BSWAP32(*((U32 *)(ptr)))
  #define RR_PUT32_BE(ptr,val)    *((U32 *)(ptr)) = RR_BSWAP32(val)
#else 
  // other platforms - don't define and error
#endif

//===================================================================
// @@ TEMP : Aliases for old names : remove me when possible :
//#define RR_GET32_OFFSET_LE	RR_GET32_LE_OFFSET
//#define RR_GET32_OFFSET_BE	RR_GET32_BE_OFFSET
//#define RR_PUT32_OFFSET_LE	RR_PUT32_LE_OFFSET
//#define RR_PUT32_OFFSET_BE	RR_PUT32_BE_OFFSET
//#define RR_GET16_OFFSET_LE	RR_GET16_LE_OFFSET
//#define RR_GET16_OFFSET_BE	RR_GET16_BE_OFFSET
//#define RR_PUT16_OFFSET_LE	RR_PUT16_LE_OFFSET
//#define RR_PUT16_OFFSET_BE	RR_PUT16_BE_OFFSET


//===================================================================
// UNALIGNED VERSIONS :

#if defined(__RADX86__) || defined(__RADPPC__) // platforms where unaligned is fast :
  #define RR_GET64_BE_UNALIGNED(ptr)                 RR_GET64_BE(ptr)
  #define RR_GET64_BE_UNALIGNED_OFFSET(ptr,offset)   RR_GET64_BE_OFFSET(ptr,offset)
  #define RR_GET32_BE_UNALIGNED(ptr)                 RR_GET32_BE(ptr)
  #define RR_GET32_BE_UNALIGNED_OFFSET(ptr,offset)   RR_GET32_BE_OFFSET(ptr,offset)
  #define RR_GET16_BE_UNALIGNED(ptr)                 RR_GET16_BE(ptr)
  #define RR_GET16_BE_UNALIGNED_OFFSET(ptr,offset)   RR_GET16_BE_OFFSET(ptr,offset)

  #define RR_GET64_LE_UNALIGNED(ptr)                 RR_GET64_LE(ptr)
  #define RR_GET64_LE_UNALIGNED_OFFSET(ptr,offset)   RR_GET64_LE_OFFSET(ptr,offset)
  #define RR_GET32_LE_UNALIGNED(ptr)                 RR_GET32_LE(ptr)
  #define RR_GET32_LE_UNALIGNED_OFFSET(ptr,offset)   RR_GET32_LE_OFFSET(ptr,offset)
  #define RR_GET16_LE_UNALIGNED(ptr)                 RR_GET16_LE(ptr)
  #define RR_GET16_LE_UNALIGNED_OFFSET(ptr,offset)   RR_GET16_LE_OFFSET(ptr,offset)
#else
  // Unaligned via bytes :
  #define RR_GET32_BE_UNALIGNED(ptr) ( \
  	( (U32)(((const U8 * RR_GET_PTR_POST)(ptr)))[0] << 24 ) | \
  	( (U32)(((const U8 * RR_GET_PTR_POST)(ptr)))[1] << 16 ) | \
  	( (U32)(((const U8 * RR_GET_PTR_POST)(ptr)))[2] << 8  ) | \
  	( (U32)(((const U8 * RR_GET_PTR_POST)(ptr)))[3] << 0  ) )

  #define RR_GET32_BE_UNALIGNED_OFFSET(ptr,offset) ( \
  	( (U32)(((const U8 * RR_GET_PTR_POST)(ptr))+(offset))[0] << 24 ) | \
  	( (U32)(((const U8 * RR_GET_PTR_POST)(ptr))+(offset))[1] << 16 ) | \
  	( (U32)(((const U8 * RR_GET_PTR_POST)(ptr))+(offset))[2] << 8  ) | \
  	( (U32)(((const U8 * RR_GET_PTR_POST)(ptr))+(offset))[3] << 0  ) )

  #define RR_GET16_BE_UNALIGNED(ptr) ( \
  	( (U16)(((const U8 * RR_GET_PTR_POST)(ptr)))[0] << 8  ) | \
  	( (U16)(((const U8 * RR_GET_PTR_POST)(ptr)))[1] << 0  ) )

  #define RR_GET16_BE_UNALIGNED_OFFSET(ptr,offset) ( \
  	( (U16)(((const U8 * RR_GET_PTR_POST)(ptr))+(offset))[0] << 8  ) | \
  	( (U16)(((const U8 * RR_GET_PTR_POST)(ptr))+(offset))[1] << 0  ) )

  #define RR_GET32_LE_UNALIGNED(ptr) ( \
  	( (U32)(((const U8 * RR_GET_PTR_POST)(ptr)))[3] << 24 ) | \
  	( (U32)(((const U8 * RR_GET_PTR_POST)(ptr)))[2] << 16 ) | \
  	( (U32)(((const U8 * RR_GET_PTR_POST)(ptr)))[1] << 8  ) | \
  	( (U32)(((const U8 * RR_GET_PTR_POST)(ptr)))[0] << 0  ) )

  #define RR_GET32_LE_UNALIGNED_OFFSET(ptr,offset) ( \
  	( (U32)(((const U8 * RR_GET_PTR_POST)(ptr))+(offset))[3] << 24 ) | \
  	( (U32)(((const U8 * RR_GET_PTR_POST)(ptr))+(offset))[2] << 16 ) | \
  	( (U32)(((const U8 * RR_GET_PTR_POST)(ptr))+(offset))[1] << 8  ) | \
  	( (U32)(((const U8 * RR_GET_PTR_POST)(ptr))+(offset))[0] << 0  ) )

  #define RR_GET16_LE_UNALIGNED(ptr) ( \
  	( (U16)(((const U8 * RR_GET_PTR_POST)(ptr)))[1] << 8  ) | \
  	( (U16)(((const U8 * RR_GET_PTR_POST)(ptr)))[0] << 0  ) )

  #define RR_GET16_LE_UNALIGNED_OFFSET(ptr,offset) ( \
  	( (U16)(((const U8 * RR_GET_PTR_POST)(ptr))+(offset))[1] << 8  ) | \
  	( (U16)(((const U8 * RR_GET_PTR_POST)(ptr))+(offset))[0] << 0  ) )

#endif

//===================================================================
// RR_ROTL32 : 32-bit rotate
//

#ifndef RR_ROTL32
  #ifdef _MSC_VER
    RADDEFFUNC unsigned long __cdecl _lrotl(unsigned long, int);
    #pragma intrinsic(_lrotl)
    #define RR_ROTL32(x,k)  _lrotl((unsigned long)(x),(int)(k))
  #else
    // NOTE(fg): Barring evidence to the contrary, let's just assume that
    // compilers are smart enough to turn this into rotates (where supported)
    // by now.
    #define RR_ROTL32(u32v,num)  ( ( (u32v) << (num) ) | ( (u32v) >> (32 - (num))) )
  #endif
#endif

//===================================================================
// RR_ROTL64 : 64-bit rotate

#ifndef RR_ROTL64
  #if ( defined(_MSC_VER) && _MSC_VER >= 1300)
    RADDEFFUNC unsigned __int64 __cdecl _rotl64(unsigned __int64 _Val, int _Shift);
    #pragma intrinsic(_rotl64)
    #define RR_ROTL64(x,k)  _rotl64((unsigned __int64)(x),(int)(k))
  #else
    // NOTE(fg): Barring evidence to the contrary, let's just assume that
    // compilers are smart enough to turn this into rotates (where supported)
    // by now.
    #define RR_ROTL64(u64v,num)  ( ( (u64v) << (num) ) | ( (u64v) >> (64 - (num))) )
  #endif
#endif

//===================================================================

//===================================================================
// some error checks :

RR_COMPILER_ASSERT( sizeof(RAD_UINTa) == sizeof( RR_STRING_JOIN(RAD_U,RAD_PTRBITS) ) );
RR_COMPILER_ASSERT( sizeof(RAD_UINTa) == RAD_PTRBYTES );
RR_COMPILER_ASSERT( RAD_TWOPTRBYTES == 2* RAD_PTRBYTES );
RR_COMPILER_ASSERT( sizeof(rrbool) == 4 );

// aliases :
#ifndef UINTn
  #define UINTn UINTa
#endif
#ifndef UINTn
  #define SINTn SINTa
#endif  
  
#ifdef __cplusplus
  #define RADDEFINEDATA extern "C"
  #define RADDECLAREDATA extern "C"

  #define RR_NAMESPACE       rr
  #define RR_NAMESPACE_START namespace RR_NAMESPACE {
  #define RR_NAMESPACE_END   };
  #define RR_NAMESPACE_USE   using namespace RR_NAMESPACE;
#else
  #define RADDEFINEDATA
  #define RADDECLAREDATA extern

  #define RR_NAMESPACE
  #define RR_NAMESPACE_START
  #define RR_NAMESPACE_END
  #define RR_NAMESPACE_USE
#endif

//===================================================================

#endif // __RADRR_CORE2H__


