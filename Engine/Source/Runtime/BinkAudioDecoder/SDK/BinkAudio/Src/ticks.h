// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef __RADRR_CORE2H__
#include "rrCore.h"
#endif

// cross platform rdtsc equiv - just for test usage.
U64 baue_ticks( void );

