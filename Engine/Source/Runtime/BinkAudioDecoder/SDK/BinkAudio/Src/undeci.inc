// Copyright Epic Games, Inc. All Rights Reserved.
static
F32 bink_Undecibel_table[ 96 ] =
{
  1.000000000000000000F, 1.165198683738708500F, 1.357688069343566900F, 1.581976413726806600F, 
  1.843316912651062000F, 2.147830486297607400F, 2.502649307250976600F, 2.916083812713623000F, 
  3.397817134857177700F, 3.959132194519043000F, 4.613175868988037100F, 5.375266551971435500F, 
  6.263253688812255900F, 7.297935009002685500F, 8.503544807434082000F, 9.908319473266601600F, 
  11.54516124725341800F, 13.45240688323974600F, 15.67472743988037100F, 18.26417160034179700F, 
  21.28139114379882800F, 24.79704856872558600F, 28.89348983764648400F, 33.66665649414062500F, 
  39.22834777832031200F, 45.70882034301757800F, 53.25985717773437500F, 62.05831909179687500F, 
  72.31027221679687500F, 84.25583648681640600F, 98.17479705810546900F, 114.3931427001953100F, 
  133.2907409667968700F, 155.3102111816406200F, 180.9672546386718700F, 210.8628082275390600F, 
  245.6970825195312500F, 286.2859191894531200F, 333.5799865722656200F, 388.6869812011718700F, 
  452.8975830078125000F, 527.7156982421875000F, 614.8936157226562500F, 716.4732666015625000F, 
  834.8337402343750000F, 972.7472534179687500F, 1133.443847656250000F, 1320.687255859375000F, 
  1538.863159179687500F, 1793.081420898437500F, 2089.296142578125000F, 2434.445068359375000F, 
  2836.612304687500000F, 3305.217285156250000F, 3851.234863281250000F, 4487.454101562500000F, 
  5228.775390625000000F, 6092.562500000000000F, 7099.046386718750000F, 8271.799804687500000F, 
  9638.290039062500000F, 11230.52343750000000F, 13085.79199218750000F, 15247.54785156250000F, 
  17766.42382812500000F, 20701.41406250000000F, 24121.25976562500000F, 28106.06250000000000F, 
  32749.14843750000000F, 38159.26562500000000F, 44463.12500000000000F, 51808.37890625000000F, 
  60367.05468750000000F, 70339.61718750000000F, 81959.63281250000000F, 95499.25781250000000F, 
  111275.6171875000000F, 129658.2031250000000F, 151077.5781250000000F, 176035.3906250000000F, 
  205116.2187500000000F, 239001.1562500000000F, 278483.8437500000000F, 324489.0312500000000F, 
  378094.1875000000000F, 440554.8750000000000F, 513333.9687500000000F, 598136.0625000000000F, 
  696947.3750000000000F, 812082.1875000000000F, 946237.1875000000000F, 1102554.375000000000F, 
  1284694.875000000000F, 1496924.875000000000F, 1744214.875000000000F, 2032357.000000000000F, 
};
/*
  2368099.750000000000F, 2759306.750000000000F, 3215140.750000000000F, 3746278.000000000000F, 
  4365158.500000000000F, 5086277.000000000000F, 5926523.500000000000F, 6905577.500000000000F, 
  8046370.000000000000F, 9375620.000000000000F, 10924461.00000000000F, 12729167.00000000000F, 
  14832010.00000000000F, 17282238.00000000000F, 20137242.00000000000F, 23463890.00000000000F, 
  27340094.00000000000F, 31856642.00000000000F, 37119320.00000000000F, 43251384.00000000000F, 
  50396456.00000000000F, 58721888.00000000000F, 68422664.00000000000F, 79726008.00000000000F, 
  92896640.00000000000F, 108243048.0000000000F, 126124656.0000000000F, 146960288.0000000000F, 
  171237936.0000000000F, 199526224.0000000000F, 232487712.0000000000F, 270894368.0000000000F, 
  315645792.0000000000F, 367790080.0000000000F, 428548512.0000000000F, 499344192.0000000000F, 
  581835200.0000000000F, 677953664.0000000000F, 789950720.0000000000F, 920449600.0000000000F, 
  1072506688.000000000F, 1249683456.000000000F, 1456129536.000000000F, 1696680192.000000000F, 
  1976969600.000000000F, 2303562496.000000000F, 2684108032.000000000F, 3127519232.000000000F, 
  3644181504.000000000F, 4246195712.000000000F, 4947661824.000000000F, 5765008896.000000000F, 
  6717381120.000000000F, 7827084288.000000000F, 9120108544.000000000F, 10626738176.00000000F, 
  12382262272.00000000F, 14427796480.00000000F, 16811249664.00000000F, 19588446208.00000000F, 
  22824433664.00000000F, 26595000320.00000000F, 30988460032.00000000F, 36107714560.00000000F, 
  42072662016.00000000F, 49023012864.00000000F, 57121550336.00000000F, 66557960192.00000000F, 
  77553246208.00000000F, 90364944384.00000000F, 105293119488.0000000F, 122687414272.0000000F, 
  142955216896.0000000F, 166571229184.0000000F, 194088583168.0000000F, 226151776256.0000000F, 
  263511752704.0000000F, 307043565568.0000000F, 357766758400.0000000F, 416869384192.0000000F, 
  485735661568.0000000F, 565978595328.0000000F, 659477495808.0000000F, 768422379520.0000000F, 
  895364759552.0000000F, 1043277873152.000000F, 1215626084352.000000F, 1416445952000.000000F, 
  1650440929280.000000F, 1923091791872.000000F, 2240784105472.000000F, 2610958696448.000000F, 
  3042285715456.000000F, 3544867405824.000000F, 4130475081728.000000F, 4812824379392.000000F, 
  5607896645632.000000F, 6534314065920.000000F, 7613774299136.000000F, 8871560085504.000000F, 
  10337130643456.00000F, 12044811108352.00000F, 14034598363136.00000F, 16353096368128.00000F, 
  19054607728640.00000F, 22202403520512.00000F, 25870213316608.00000F, 30143938560000.00000F, 
  35123678412800.00000F, 40926063886336.00000F, 47687000915968.00000F, 55564830769152.00000F, 
  64744069267456.00000F, 75439708045312.00000F, 87902251909120.00000F, 102423595057152.0000F, 
  119343836823552.0000F, 139059288604672.0000F, 162031701327872.0000F, 188799128698880.0000F, 
  219988510113792.0000F, 256330325557248.0000F, 298675783860224.0000F, 348016636592128.0000F, 
  405508531945472.0000F, 472498042830848.0000F, 550554107379712.0000F, 641504905265152.0000F, 
  747480740265984.0000F, 870963600162816.0000F, 1014845675667456.000F, 1182496838385664.000F, 
  1377843896385536.000F, 1605461862252544.000F, 1870682132578304.000F, 2179716438032384.000F, 
  2539802872250368.000F, 2959375006171136.000F, 3448259859185664.000F, 4017908082540544.000F, 
  4681661455269888.000F, 5455065879937024.000F, 6356235917983744.000F, 7406277575573504.000F, 
  8629785446055936.000F, 10055415014359040.00F, 11716556966854656.00F, 13652117264793600.00F, 
  15907429287985152.00F, 18535315765460992.00F, 21597326799798272.00F, 25165177607421952.00F, 
  29322433219526656.00F, 34166460544712704.00F, 39810717586554880.00F, 46387394848489472.00F, 
  54050535627554816.00F, 62979613722148864.00F, 73383768269783040.00F, 85506674239995904.00F, };
*/

