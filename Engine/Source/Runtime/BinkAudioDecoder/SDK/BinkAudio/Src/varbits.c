// Copyright Epic Games, Inc. All Rights Reserved.
#include "rrCore.h"
#include "varbits.h"

RRSTRIPPABLEPUB 
RADDEFINEDATA const RAD_ALIGN(U32, VarBitsLens[ 33 ], 32 ) =
{
           0,         1,           3,          7,
         0xf,      0x1f,        0x3f,       0x7f,
        0xff,     0x1ff,       0x3ff,      0x7ff,
       0xfff,    0x1fff,      0x3fff,     0x7fff,
      0xffff,    0x1ffff,    0x3ffff,    0x7ffff,
     0xfffff,   0x1fffff,   0x3fffff,   0x7fffff,
    0xffffff,  0x1ffffff,  0x3ffffff,  0x7ffffff,
   0xfffffff, 0x1fffffff, 0x3fffffff, 0x7fffffff,
  0xffffffff
};

RADDEFFUNC void VarBitsCopy(VARBITS* dest,VARBITS* src,U32 size)
{
  U32 val;
  while (size>=8) {
    VarBitsGet(val,U32,*src,8);
    VarBitsPut(*dest,val,8);
    size-=8;
  }

  if (size) {
    VarBitsGet(val,U32,*src,size);
    VarBitsPut(*dest,val,size);
  }
}


