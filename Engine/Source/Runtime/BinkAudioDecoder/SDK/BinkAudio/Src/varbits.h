// Copyright Epic Games, Inc. All Rights Reserved.
#ifndef __VARBITSH__
#define __VARBITSH__

#ifndef __RADRR_CORE2H__
#include "rrCore.h"
#endif

#ifdef WRAP_PUBLICS
#define rfmerge3(name,add) name##add
#define rfmerge2(name,add) rfmerge3(name,add)
#define rfmerge(name)      rfmerge2(name,WRAP_PUBLICS)
#define VarBitsCopy                          rfmerge(VarBitsCopy)
#define VarBitsLens                          rfmerge(VarBitsLens)
#endif


// variable bit macros
//#if defined(__RADX86__)
RADDECLAREDATA const RAD_ALIGN(U32,VarBitsLens[33],32);
//#endif

/*************

VARBITS can either be in a struct , or in local variables set up by VARBITSLOCAL(name)

the "Local" macros make the names of the local variables using ##

***************/

//#define USE64BITVB
#ifdef USE64BITVB

#define BITSTYPE U64
#define BITSTYPELEN 64
#define BITSTYPEBYTES 8
#define BITSTOPMASK    (1UL64<<(BITSTYPELEN-1))

//NOTE this is read-only on USE64BITVB!
#define VarBitsOpen(vb,pointer) { (vb).init=pointer; if (((U32)pointer)&4) { (vb).bits = *((U32* RADRESTRICT )pointer); (vb).cur = ((char*)pointer)+4; (vb).bitlen = 32; } else { (vb).cur=pointer; (vb).bits=(vb).bitlen=0; } }
#define VarBitsLocalOpen(vb,pointer) { if (((U32)pointer)&4) { vb##bits = *((U32 * RADRESTRICT)pointer); vb##cur = ((char*)pointer)+4; vb##bitlen = 32; } else { vb##cur=pointer; vb##bits=vb##bitlen=0; } }

#else

#define BITSTYPE U32
#define BITSTYPELEN 32
#define BITSTYPEBYTES 4
#define BITSTOPMASK    (1UL<<(BITSTYPELEN-1))

#define VarBitsOpen(vb,pointer) { (vb).cur=(vb).init=pointer; (vb).bits=(vb).bitlen=0; }
#define VarBitsLocalOpen(vb,pointer) { vb##cur=pointer; vb##bits=vb##bitlen=0; }

#endif

#define VARBITSTEMP BITSTYPE

typedef struct _VARBITS
{
  BITSTYPE bits;
  void* RADRESTRICT cur;
  U32 bitlen;
  void* RADRESTRICT init;
} VARBITS;

// FG: same as VARBITS (and can be used with the same macros), but has space for an "end" pointer
// not used by the macros, but useful if you need to propagate the "end" pointer downstream in
// a decoder.
typedef struct _VARBITSEND
{
  BITSTYPE bits;
  void* RADRESTRICT cur;
  U32 bitlen;
  void* RADRESTRICT init;
  void* RADRESTRICT end;
} VARBITSEND;

// CB : WARNING : these functions are generally NOT safe to call with # bits = 0

#define VarBitsPut(vb,val,size) { U32 __s=size; U32 __v=(val)&VarBitsLens[__s]; (vb).bits|=__v<<((vb).bitlen); (vb).bitlen+=__s; if ((vb).bitlen>=32) { *((U32*)(vb).cur)=(vb).bits; (vb).cur=((char*)((vb).cur)+4); (vb).bitlen-=32; (vb).bits=0; if ((vb).bitlen) { (vb).bits=__v>>(__s-(vb).bitlen); } } }
#define VarBitsPut1(vb,boolean) { if (boolean) (vb).bits|=(1<<(vb).bitlen); if ((++(vb).bitlen)==32) { *((U32*)(vb).cur)=(vb).bits; (vb).cur=((char*)((vb).cur)+4); (vb).bits=(vb).bitlen=0; } }
#define VarBitsPuta1(vb) { (vb).bits|=(1<<(vb).bitlen); if ((++(vb).bitlen)==32) { *((U32*)(vb).cur)=(vb).bits; (vb).cur=((char*)((vb).cur)+4); (vb).bits=(vb).bitlen=0; } }
#define VarBitsPuta0(vb) { if ((++(vb).bitlen)==32) { *((U32*)(vb).cur)=(vb).bits; (vb).cur=((char*)((vb).cur)+4); (vb).bits=(vb).bitlen=0; } }
#define VarBitsPutAlign(vb) { U32 __s2=(32-(vb).bitlen)&31; if (__s2) { VarBitsPut((vb),0,__s2);  } }
#define VarBitsFlushtoMemOnly(vb) { if (((vb).bitlen)) { *((U32*)(vb).cur)=(vb).bits; } }
#define VarBitsConvertPutToGet(gvb,pvb) { if (((pvb).bitlen)) { (gvb).bits=(*((U32*)(pvb).cur))>>(pvb).bitlen; (gvb).bitlen=BITSTYPELEN-(pvb).bitlen; (gvb).cur=((char*)((pvb).cur)+4); } else { (gvb).bits=0; (gvb).bitlen=0; (gvb).cur=(pvb).cur; } (gvb).init=(pvb).init; }
#define VarBitsFlush(vb) VarBitsPutAlign(vb)
#define VarBitsSize(vb) ((U32)( (((char*)(vb).cur)-((char*)(vb).init))*8 +(vb).bitlen ))    // in bits !!

// VarBitsCopy size is in bits of course; don't use this for big copies
RADDEFFUNC void VarBitsCopy(VARBITS* dest,VARBITS* src,U32 size);

// getbitlevel :
//    getbitlevel(n) is the number of bits that n uses for its on bits
//    eg. n < (1<<getbitlevel(n)) , n >= (1<<(getbitlevel(n)-1))
//    getbitlevel(n)-1 is the bit position of the leftmost 1 bit in 'n'
// NOTE : getbitlevel(0) = 0
//  getbitlevel(n) = ilog2ceil except on powers of two
//---------------------------------------


// WARNING : this getbitlevel only works on *constant* U16 values ! (up to 65535)
//   it will fail to compile with strange errors if you use a variable
#define getbitlevelconst(level)        \
(                                      \
  (((level)<    1)?0:                  \
  (((level)<    2)?1:                  \
  (((level)<    4)?2:                  \
  (((level)<    8)?3:                  \
  (((level)<   16)?4:                  \
  (((level)<   32)?5:                  \
  (((level)<   64)?6:                  \
  (((level)<  128)?7:                  \
  (((level)<  256)?8:                  \
  (((level)<  512)?9:                  \
  (((level)< 1024)?10:                 \
  (((level)< 2048)?11:                 \
  (((level)< 4096)?12:                 \
  (((level)< 8192)?13:                 \
  (((level)<16384)?14:                 \
  (((level)<32768)?15:                 \
  (((level)<65536)?16:sizeof(char[65535-level]) \
  )))))))))))))))))                    \
)


#if defined(__RADPPC__)

  #if defined(__GNUC__)

    #if defined(__SNC__)
      #define count_leading_zeros(count, x) count = __cntlzw(x)
    #elif defined(__ghs__)
      RADDEFFUNC unsigned int __CLZ32(unsigned int a);
      #define count_leading_zeros(count, x) count = __CLZ32(x)
    #else
      #define count_leading_zeros(count, x)     \
            __asm__ ("{cntlz|cntlzw} %0,%1"     \
               : "=r" (count)                   \
                : "r" (x))
    #endif


    static RADINLINE U32 getbitlevelvar( register U32 n )
    {
      count_leading_zeros( n, n );
      return( 32 - n );
    }

  #else

    #ifdef _MSC_VER
      #include <ppcintrinsics.h>
      #define __cntlzw _CountLeadingZeros
    #endif

    #define getbitlevelvar(n) (U32) (32 - __cntlzw(n))

  #endif

#elif defined(__RADSPU__)

  static RADINLINE U32 getbitlevelvar( register U32 n )
  {
    vector unsigned int v;
    v[0]=n;
    v = __builtin_spu_cntlz( v );  
    return( 32 - v[0] );
  }

#elif defined(__RADARM__)

  #ifdef _MSC_VER

    #ifdef _M_ARM64

        #define getbitlevelvar(n) (U32) (32 - _CountLeadingZeros(n)) 

    #else

       #define getbitlevelvar(n) (U32) (32 - _arm_clz(n))

    #endif

  #else

    #define getbitlevelvar(n) (U32) (32 - __builtin_clz(n))

  #endif

#elif (_MSC_FULL_VER >= 13012035 )

    RADDEFSTART
    unsigned char _BitScanReverse(unsigned long* Index, unsigned long Mask);
    #pragma intrinsic(_BitScanReverse)
    RADDEFEND

    static RADINLINE U32 getbitlevelvar( register U32 val )
    {
      if ( val )
      {
        U32 b = 0;
        _BitScanReverse( (unsigned long*)&b, val );
        return b + 1;
      }
      return 0;
    }

#else

    static RADINLINE U32 getbitlevelvar( register U32 val )
    {
      static char vs[16]={0,1,2,2,3,3,3,3,4,4,4,4,4,4,4,4};
      int bits=0;

      if ( val & 0xffff0000 )
      {
        bits = 16;
        val >>= 16;
      }

      if ( val & 0xff00 )
      {
        bits += 8;
        val >>= 8;
      }

      if ( val & 0xf0 )
      {
        bits += 4;
        val >>= 4;
      }

      bits += vs[ val ];

      return bits;
    }

#endif

#define VarBitsGetAlign(vb) { (vb).bitlen=0; }
#define VarBitsPos(vb) ((U32)( (((U8*)(vb).cur)-((U8*)(vb).init))*8-(vb).bitlen ))

// CB : GetBitsLen(bits) = (1<<bits)-1
//   except that values for bits=32 , GetBitsLen fails for bits=0
// don't pass zero to this function
#define GetBitsLen(val) (((U32)0xffffffff)>>(U32)(32-(val)))
// for debugging: causes crash on zero: #define GetBitsLen(val) (((val)==0)?(((U8*)val)[0]=0):(0xffffffffL>>(U32)(32-(val))))

// get 1 bit
//    best to use like : if ( VarBitsGet1(vb,i) )
// 
// This *doesn't* set i to the result - that's just a temp - the expression is the result!!!
//    i will be the unmasked bits, but that's not guaranteed!
#define VarBitsGet1(vb,i)             \
(                                     \
  ((vb).bitlen==0)?                   \
  (                                   \
    i=*((BITSTYPE* RADRESTRICT)((vb).cur)),       \
    ((vb).cur)=((char*)((vb).cur))+BITSTYPEBYTES, \
    ((vb).bits)=((BITSTYPE)i)>>1,     \
    ((vb).bitlen)=(BITSTYPELEN-1)     \
  ):(                                 \
    i=((vb).bits),                    \
    ((vb).bits)>>=1,                  \
    --((vb).bitlen)                   \
  ),(i&1)                             \
)

//not USE64BITVB safe!
#define VarBitsGet1LE(vb,i)           \
(                                     \
  ((vb).bitlen==0)?                   \
  (                                   \
    i=radloadu32ptr((vb).cur),        \
    ((vb).cur)=((char*)((vb).cur))+4, \
    ((vb).bits)=((U32)i)>>1,          \
    ((vb).bitlen)=31                  \
  ):(                                 \
    i=((vb).bits),                    \
    ((vb).bits)>>=1,                  \
    --((vb).bitlen)                   \
  ),(i&1)                             \
)


#define VarBitsGet(v,typ,vb,len)                                \
{                                                               \
  if (((vb).bitlen)<(len)) {                                    \
    register BITSTYPE nb=*((BITSTYPE* RADRESTRICT)((vb).cur));  \
    v=(typ)((((vb).bits)|(nb<<((vb).bitlen)))&GetBitsLen(len)); \
    ((vb).bits)=nb>>((len)-((vb).bitlen));                      \
    ((vb).bitlen)=((vb).bitlen)+BITSTYPELEN-(len);              \
    ((vb).cur)=((char*)((vb).cur))+BITSTYPEBYTES;               \
  } else {                                                      \
    v=(typ)(((vb).bits)&GetBitsLen(len));                       \
    ((vb).bits)>>=(len);                                        \
    ((vb).bitlen)-=(len);                                       \
  }                                                             \
}

#define VarBitsGetWithCheck(v,typ,vb,len,endp,dowhat)           \
{                                                               \
  if (((vb).bitlen)<(len)) {                                    \
    register BITSTYPE nb;                                       \
    if ( ( (U8*)((vb).cur) ) >= ( (U8*) (endp) ) ) dowhat       \
    nb=*((BITSTYPE* RADRESTRICT)((vb).cur));                    \
    v=(typ)((((vb).bits)|(nb<<((vb).bitlen)))&GetBitsLen(len)); \
    ((vb).bits)=nb>>((len)-((vb).bitlen));                      \
    ((vb).bitlen)=((vb).bitlen)+BITSTYPELEN-(len);              \
    ((vb).cur)=((char*)((vb).cur))+BITSTYPEBYTES;               \
  } else {                                                      \
    v=(typ)(((vb).bits)&GetBitsLen(len));                       \
    ((vb).bits)>>=(len);                                        \
    ((vb).bitlen)-=(len);                                       \
  }                                                             \
}

#define VarBitsGetWithCheckBE(v,typ,vb,len,endp,dowhat)         \
{                                                               \
  if (((vb).bitlen)<(len)) {                                    \
    register BITSTYPE nb;                                       \
    if ( ( (U8*)((vb).cur) ) >= ( (U8*) (endp) ) ) dowhat       \
    nb=radloadu32ptrBE((vb).cur);                               \
    v=(typ)((((vb).bits)|(nb<<((vb).bitlen)))&GetBitsLen(len)); \
    ((vb).bits)=nb>>((len)-((vb).bitlen));                      \
    ((vb).bitlen)=((vb).bitlen)+BITSTYPELEN-(len);              \
    ((vb).cur)=((char*)((vb).cur))+BITSTYPEBYTES;               \
  } else {                                                      \
    v=(typ)(((vb).bits)&GetBitsLen(len));                       \
    ((vb).bits)>>=(len);                                        \
    ((vb).bitlen)-=(len);                                       \
  }                                                             \
}

#define VarBitsGetWithCheckLE(v,typ,vb,len,endp,dowhat)         \
{                                                               \
  if (((vb).bitlen)<(len)) {                                    \
    register BITSTYPE nb;                                       \
    if ( ( (U8*)((vb).cur) ) >= ( (U8*) (endp) ) ) dowhat       \
    nb=radloadu32ptr((vb).cur);                                 \
    v=(typ)((((vb).bits)|(nb<<((vb).bitlen)))&GetBitsLen(len)); \
    ((vb).bits)=nb>>((len)-((vb).bitlen));                      \
    ((vb).bitlen)=((vb).bitlen)+BITSTYPELEN-(len);              \
    ((vb).cur)=((char*)((vb).cur))+BITSTYPEBYTES;               \
  } else {                                                      \
    v=(typ)(((vb).bits)&GetBitsLen(len));                       \
    ((vb).bits)>>=(len);                                        \
    ((vb).bitlen)-=(len);                                       \
  }                                                             \
}

// Peek : put len bits in v but don't shift the read pointer
#define VarBitsPeek(v,typ,vb,len)                               \
{                                                               \
  if (((vb).bitlen)<(len)) {                                    \
    register BITSTYPE nb=*((BITSTYPE* RADRESTRICT)((vb).cur));              \
    v=(typ)((((vb).bits)|(nb<<((vb).bitlen)))&GetBitsLen(len)); \
  } else {                                                      \
    v=(typ)(((vb).bits)&GetBitsLen(len));                       \
  }                                                             \
}

//not USE64BITVB safe!
#define VarBitsGetLE(v,typ,vb,len)                              \
{                                                               \
  if (((vb).bitlen)<(len)) {                                    \
    register U32 nb=radloadu32ptr((vb).cur);                    \
    v=(typ)((((vb).bits)|(nb<<((vb).bitlen)))&GetBitsLen(len)); \
    ((vb).bits)=nb>>((len)-((vb).bitlen));                      \
    ((vb).bitlen)=((vb).bitlen)+32-(len);                       \
    ((vb).cur)=((char*)((vb).cur))+4;                           \
  } else {                                                      \
    v=(typ)(((vb).bits)&GetBitsLen(len));                       \
    ((vb).bits)>>=(len);                                        \
    ((vb).bitlen)-=(len);                                       \
  }                                                             \
}

// VarBitsUse : just skip 'len' bits ; useful after a Peek
#define VarBitsUse(vb,len)                                      \
{                                                               \
  if (((vb).bitlen)<(len)) {                                    \
    register BITSTYPE nb=*((BITSTYPE* RADRESTRICT)((vb).cur));  \
    ((vb).bits)=nb>>((len)-((vb).bitlen));                      \
    ((vb).bitlen)=((vb).bitlen)+BITSTYPELEN-(len);              \
    ((vb).cur)=((char*)((vb).cur))+BITSTYPEBYTES;               \
  } else {                                                      \
    ((vb).bits)>>=(len);                                        \
    ((vb).bitlen)-=(len);                                       \
  }                                                             \
}

//-------------------------------------------------------------------------------------
// VarBitsLocal stuff for even more speed

// VARBITSLOCAL defines the local vars to hold the varbits goodies
//    VarBitsCopyToLocal() at the start
//    VarBitsCopyFromLocal() at the end

#define VARBITSLOCAL(name) void * name##cur; BITSTYPE name##bits; U32 name##bitlen


#ifdef __RAD64REGS__

#ifdef INC_BINK2

#if defined(__RADX86__)
#include <xmmintrin.h>
#define VBPREFETCH( ptr ) _mm_prefetch(((char*)(ptr))+1024, _MM_HINT_T0 )
#elif defined(__RADARM64__)
// should we do this??
#define VBPREFETCH( ptr ) __builtin_prefetch(((char*)(ptr))+1024)
#else
#define VBPREFETCH( ptr )
#endif
#else
#define VBPREFETCH( ptr )
#endif

#define VarBitsLocalGet(v,typ,vb,len)                           \
{                                                               \
  register U64 _bits = (vb##bits);                              \
  if ((vb##bitlen)<len) {                                       \
    _bits |= (((U64)(*((U32* RADRESTRICT)(vb##cur))))<<(vb##bitlen)); \
    (vb##bitlen)=(vb##bitlen)+32;                               \
    VBPREFETCH((vb##cur));                                      \
    (vb##cur)=((char*)(vb##cur))+4;                             \
  }                                                             \
  (vb##bits)=(U32)(_bits>>(len));                               \
  (vb##bitlen)-=(len);                                          \
  v=(typ)(((U32)_bits)&GetBitsLen(len));                        \
}

// mask == GetBitsLen(len) - for when you know this in advance.
// load == what to do to load
#define VarBitsLocalGetWithCheckBase(v,typ,vb,len,mask,endp,dowhat,load) \
{                                                               \
  register U64 _bits = (vb##bits);                              \
  if ((vb##bitlen)<len) {                                       \
    if ( ( (U8*)(vb##cur) ) >= ( (U8*) (endp) ) ) dowhat        \
    _bits|=((U64) (load(vb##cur))) << (vb##bitlen);             \
    (vb##bitlen)+=32;                                           \
    (vb##cur)=((char*)(vb##cur))+4;                             \
  }                                                             \
  v=(typ)(_bits & (mask));                                      \
  (vb##bits)=(U32)(_bits>>(len));                               \
  (vb##bitlen)-=(len);                                          \
}

#else

#define VarBitsLocalGet(v,typ,vb,len)                           \
{                                                               \
  if ((vb##bitlen)<len) {                                       \
    register BITSTYPE nb=*((BITSTYPE* RADRESTRICT)(vb##cur));   \
    v=(typ)(((vb##bits)|(nb<<(vb##bitlen)))&GetBitsLen(len));   \
    (vb##bits)=nb>>((len)-(vb##bitlen));                        \
    (vb##bitlen)=(vb##bitlen)+BITSTYPELEN-(len);                \
    (vb##cur)=((char*)(vb##cur))+BITSTYPEBYTES;                 \
  } else {                                                      \
    v=(typ)((vb##bits)&GetBitsLen(len));                        \
    (vb##bits)>>=(len);                                         \
    (vb##bitlen)-=(len);                                        \
  }                                                             \
}

// mask == GetBitsLen(len) - for when you know this in advance.
// load == what to do to load
#define VarBitsLocalGetWithCheckBase(v,typ,vb,len,mask,endp,dowhat,load) \
{                                                               \
  if ((vb##bitlen)<len) {                                       \
    register BITSTYPE nb;                                       \
    if ( ( (U8*)(vb##cur) ) >= ( (U8*) (endp) ) ) dowhat        \
    nb=load(vb##cur);                                           \
    v=(typ)(((vb##bits)|(nb<<(vb##bitlen)))&(mask));            \
    (vb##bits)=nb>>((len)-(vb##bitlen));                        \
    (vb##bitlen)=(vb##bitlen)+BITSTYPELEN-(len);                \
    (vb##cur)=((char*)(vb##cur))+BITSTYPEBYTES;                 \
  } else {                                                      \
    v=(typ)((vb##bits)&(mask));                                 \
    (vb##bits)>>=(len);                                         \
    (vb##bitlen)-=(len);                                        \
  }                                                             \
}

#endif

#define radloadbitsnative(ptr) (*((BITSTYPE* RADRESTRICT)(ptr)))

#define VarBitsLocalGetWithCheck(v,typ,vb,len,endp,dowhat)      VarBitsLocalGetWithCheckBase(v,typ,vb,len,GetBitsLen(len),endp,dowhat,radloadbitsnative)
#define VarBitsLocalGetWithCheckBE(v,typ,vb,len,endp,dowhat)    VarBitsLocalGetWithCheckBase(v,typ,vb,len,GetBitsLen(len),endp,dowhat,radloadu32ptrBE)
#define VarBitsLocalGetWithCheckLE(v,typ,vb,len,endp,dowhat)    VarBitsLocalGetWithCheckBase(v,typ,vb,len,GetBitsLen(len),endp,dowhat,radloadu32ptr)

// same as LocalGetWithCheck, but passing in the bit mask yourself
#define VarBitsLocalGetWithCheckM(v,typ,vb,len,mask,endp,dowhat)      VarBitsLocalGetWithCheckBase(v,typ,vb,len,mask,endp,dowhat,radloadbitsnative)
#define VarBitsLocalGetWithCheckMBE(v,typ,vb,len,mask,endp,dowhat)    VarBitsLocalGetWithCheckBase(v,typ,vb,len,mask,endp,dowhat,radloadu32ptrBE)
#define VarBitsLocalGetWithCheckMLE(v,typ,vb,len,mask,endp,dowhat)    VarBitsLocalGetWithCheckBase(v,typ,vb,len,mask,endp,dowhat,radloadu32ptr)

// just the refill for a VarBitsLocalGet1WithCheck, and with an extra conditional
// this turns out to be really useful to have in Bink Audio.
#define VarBitsLocalFill1WithCheckBase(vb,endp,cond,dowhat,load)\
{                                                               \
  if ((vb##bitlen)==0 && (cond)) {                              \
    if ( ( (U8*)(vb##cur) ) >= ( (U8*) (endp) ) ) dowhat        \
    (vb##bits)=load(vb##cur);                                   \
    (vb##cur)=((char*)(vb##cur))+BITSTYPEBYTES;                 \
    (vb##bitlen)=(BITSTYPELEN);                                 \
  }                                                             \
}

#define VarBitsLocalGet1WithCheckBase(v,vb,endp,dowhat,load)    \
{                                                               \
  VarBitsLocalFill1WithCheckBase(vb,endp,1,dowhat,load)         \
  --(vb##bitlen);                                               \
  v=(vb##bits);                                                 \
  (vb##bits)>>=1;                                               \
  v&=1;                                                         \
}

#define VarBitsLocalFill1WithCheck(vb,endp,cond,dowhat)         VarBitsLocalFill1WithCheckBase(vb,endp,cond,dowhat,radloadbitsnative)
#define VarBitsLocalFill1WithCheckBE(vb,endp,cond,dowhat)       VarBitsLocalFill1WithCheckBase(vb,endp,cond,dowhat,radloadu32ptrBE)
#define VarBitsLocalFill1WithCheckLE(vb,endp,cond,dowhat)       VarBitsLocalFill1WithCheckBase(vb,endp,cond,dowhat,radloadu32ptr)

#define VarBitsLocalGet1WithCheck(v,vb,endp,dowhat)             VarBitsLocalGet1WithCheckBase(v,vb,endp,dowhat,radloadbitsnative)
#define VarBitsLocalGet1WithCheckBE(v,vb,endp,dowhat)           VarBitsLocalGet1WithCheckBase(v,vb,endp,dowhat,radloadu32ptrBE)
#define VarBitsLocalGet1WithCheckLE(v,vb,endp,dowhat)           VarBitsLocalGet1WithCheckBase(v,vb,endp,dowhat,radloadu32ptr)


// get the value, if the next bit is X (mask bits with mask)
//   usually, you will use the two wrapper macros below

#define VarBitsLocalGetIfxSM(v,typ,vb,len,i,x,mask)            \
  (vb##bitlen==0)?                                             \
  (                                                            \
    i=*((BITSTYPE* RADRESTRICT)(vb##cur)),                     \
    (vb##cur)=((char*)(vb##cur))+BITSTYPEBYTES,                \
    ( ( i & 1 ) == x ) ?                                       \
    (                                                          \
      v = (typ)( ( i >> 1 ) mask ),                            \
      (vb##bits)=((BITSTYPE)i)>>(len+1),                       \
      (vb##bitlen)=(BITSTYPELEN - 1 - len),                    \
      x                                                        \
    ):(                                                        \
      (vb##bits)=(((BITSTYPE)i)>>1),                           \
      (vb##bitlen)=(BITSTYPELEN-1),                            \
      !x                                                       \
    )                                                          \
  ):(                                                          \
    --(vb##bitlen),                                            \
    ( ( (vb##bits) & 1 ) == x ) ?                              \
    (                                                          \
      ( ( vb##bitlen ) < len )?                                \
      (                                                        \
        i=*((BITSTYPE* RADRESTRICT)(vb##cur)),                 \
       (vb##cur)=(((char*)(vb##cur))+BITSTYPEBYTES ),          \
        v=(typ)(((vb##bits>>1)|(i<<vb##bitlen)) mask ),        \
       (vb##bits)=(i>>((len)-vb##bitlen)),                     \
       (vb##bitlen)=((vb##bitlen)+BITSTYPELEN-(len)),          \
       x                                                       \
      ):(                                                      \
        v = (typ) ( ( (vb##bits) >> 1 ) mask ),                \
        (vb##bits)>>=(len+1),                                  \
        (vb##bitlen)-=len,                                     \
        x                                                      \
      )                                                        \
    ):(                                                        \
      (vb##bits)>>=1,                                          \
      !x                                                       \
    )                                                          \
  )                                                            \


// get the value, if the next bit is x (0 or 1)

#define VarBitsLocalGetIfx(v,typ,vb,len,i,x)            \
  VarBitsLocalGetIfxSM(v,typ,vb,len,i,x,&GetBitLen(len))

// get the value, if the next bit is x (0 or 1), but don't
//   mask off the high bits of the value

#define VarBitsLocalGetIfxNM(v,typ,vb,len,i,x)            \
  VarBitsLocalGetIfxSM(v,typ,vb,len,i,x, )


#define VarBitsLocalPeek(v,typ,vb,len)                          \
{                                                               \
  if ((vb##bitlen)<(len)) {                                     \
    register BITSTYPE nb=*((BITSTYPE* RADRESTRICT)(vb##cur));   \
    v=(typ)(((vb##bits)|(nb<<(vb##bitlen)))&GetBitsLen(len));   \
  } else {                                                      \
    v=(typ)((vb##bits)&GetBitsLen(len));                        \
  }                                                             \
}

#define VarBitsLocalGet1BE(vb,i)                               \
(                                                              \
  (vb##bitlen==0)?                                             \
  (                                                            \
    i=radloadu32ptrBE(vb##cur),                                \
    (vb##cur)=((char*)(vb##cur))+BITSTYPEBYTES,                \
    (vb##bits)=((BITSTYPE)i)>>1,                               \
    (vb##bitlen)=(BITSTYPELEN-1)                               \
  ):(                                                          \
    i=(vb##bits),                                              \
    (vb##bits)>>=1,                                            \
    --(vb##bitlen)                                             \
  ),(i&1)                                                      \
)

#define VarBitsLocalGet1LE(vb,i)                               \
(                                                              \
  (vb##bitlen==0)?                                             \
  (                                                            \
    i=radloadu32ptr(vb##cur),                                  \
    (vb##cur)=((char*)(vb##cur))+BITSTYPEBYTES,                \
    (vb##bits)=((BITSTYPE)i)>>1,                               \
    (vb##bitlen)=(BITSTYPELEN-1)                               \
  ):(                                                          \
    i=(vb##bits),                                              \
    (vb##bits)>>=1,                                            \
    --(vb##bitlen)                                             \
  ),(i&1)                                                      \
)

// if imask is -1, then read a bit and then &-it with input mask
//   so now you have a mask that you can (v^m)-m to neg a value
#define VarBitsLocalMaskFromMaskAndUse1(omask,vb,imask)     \
  if (vb##bitlen==0)                                   \
  {                                                    \
    (vb##bits)=*((BITSTYPE* RADRESTRICT)(vb##cur));    \
    (vb##bitlen)=BITSTYPELEN;                          \
    (vb##cur)=((char*)(vb##cur))+BITSTYPEBYTES;        \
  }                                                    \
  (vb##bitlen)+=imask;                                 \
  omask=(((S32)(vb##bits)<<31)>>31)&imask;             \
  (vb##bits)=((vb##bits>>1)&imask)|(vb##bits&~imask);  \

#ifdef __RAD64REGS__

#define VarBitsLocalGet1(vb,i)                                 \
(                                                              \
  ((vb##bitlen==0)?                                            \
  (                                                            \
    (vb##bits)=*((BITSTYPE* RADRESTRICT)(vb##cur)),            \
    (vb##cur)=((char*)(vb##cur))+4,                            \
    (vb##bitlen)=32                                            \
  ):0),                                                        \
  (                                                            \
    i=(vb##bits)&1,                                            \
    (vb##bits)>>=1,                                            \
    --(vb##bitlen)                                             \
  ),(i)                                                        \
)

#else

#define VarBitsLocalGet1(vb,i)                                 \
(                                                              \
  (vb##bitlen==0)?                                             \
  (                                                            \
    i=*((BITSTYPE* RADRESTRICT)(vb##cur)),                     \
    (vb##cur)=((char*)(vb##cur))+BITSTYPEBYTES,                \
    (vb##bits)=((BITSTYPE)i)>>1,                               \
    (vb##bitlen)=(BITSTYPELEN-1)                               \
  ):(                                                          \
    i=(vb##bits),                                              \
    (vb##bits)>>=1,                                            \
    --(vb##bitlen)                                             \
  ),(i&1)                                                      \
)

#endif

#define VarBitsLocalUse(vb,len)                                \
{                                                              \
  if ((vb##bitlen)<(len)) {                                    \
    register BITSTYPE nb=*((BITSTYPE* RADRESTRICT)(vb##cur));  \
    (vb##bits)=nb>>((len)-(vb##bitlen));                       \
    (vb##bitlen)=(vb##bitlen)+BITSTYPELEN-(len);               \
    (vb##cur)=((char*)(vb##cur))+BITSTYPEBYTES;                \
  } else {                                                     \
    (vb##bits)>>=(len);                                        \
    (vb##bitlen)-=(len);                                       \
  }                                                            \
}

#define VarBitsLocalPos(vb,origvb) ((U32)( (((char*)vb##cur)-((char*)(origvb)->init))*8 +(32-vb##bitlen) ))    // in bits !!



#define VarBitsCopyToLocal( local, vb ) local##cur = (vb).cur; local##bits = (vb).bits; local##bitlen = (vb).bitlen;

#define VarBitsCopyFromLocal( vb, local )  (vb).cur = local##cur;  (vb).bits = local##bits;  (vb).bitlen = local##bitlen;




// classifies a signed value into: 0 = zero, 1 = neg, 2 = pos
#define CLASSIFY_SIGN( val )  ( (((U32)((S32)(val))) >> 31 ) + ((((U32)(-(S32)(val))) >> 30 ) & 2 ) )

//=========================================================================================================

// VARBITSB = backward !
//
// CB : these *do* work with len == 0 , VarBits does not

typedef struct _VARBITSB
{
  BITSTYPE bits;
  void* RADRESTRICT cur;
  U32 bitlen;
  void* RADRESTRICT init;
  int isPut;  // CB maybe temp ; help me find bugs
} VARBITSB;

#define VarBitsBPutOpen(vb,ptr)    { (vb).cur=(vb).init=(void *)(ptr); (vb).bits=(vb).bitlen=0; (vb).isPut = 1; }
#define VarBitsBGetOpen(vb,ptr)    { (vb).cur=(vb).init=(void *)(ptr); (vb).bits=(vb).bitlen=0; (vb).isPut = 0; }

// __v=(val)&VarBitsLens[__s]
#define VarBitsBPut(vb,val,size) do { U32 __s=size; U32 __v=(val);              \
            radassert( __v < (1UL<<__s) );                                      \
if ((vb).bitlen+__s >=BITSTYPELEN) { U32 __r = __s + (vb).bitlen - BITSTYPELEN; \
 (vb).bits <<= (BITSTYPELEN-(vb).bitlen); (vb).bits |= __v >> __r;              \
 *((U32*)(vb).cur)=(vb).bits; (vb).cur=((char*)((vb).cur)+4);                   \
(vb).bitlen = __r; (vb).bits = __v&VarBitsLens[__r]; }                          \
else  { (vb).bits = ((vb).bits<<__s) | __v; (vb).bitlen+=__s; } } while(0)

#define VarBitsBPuta1(vb) { (vb).bits = (vb).bits + (vb).bits + 1; if ((++(vb).bitlen)==BITSTYPELEN) { *((U32*)(vb).cur)=(vb).bits; (vb).cur=((char*)((vb).cur)+4); (vb).bits=(vb).bitlen=0; } }
#define VarBitsBPuta0(vb) { (vb).bits <<= 1; if ((++(vb).bitlen)==BITSTYPELEN) { *((U32*)(vb).cur)=(vb).bits; (vb).cur=((char*)((vb).cur)+4); (vb).bits=(vb).bitlen=0; } }
#define VarBitsBPut1(vb,bit) { (vb).bits = (vb).bits + (vb).bits + ((bit)?1:0); if ((++(vb).bitlen)==BITSTYPELEN) { *((U32*)(vb).cur)=(vb).bits; (vb).cur=((char*)((vb).cur)+4); (vb).bits=(vb).bitlen=0; } }

#define VarBitsBPutAlign(vb) { U32 __s2=(32-(vb).bitlen)&31; if (__s2) { VarBitsBPut((vb),0,__s2);  } }
#define VarBitsBPutFlush(vb)    VarBitsBPutAlign(vb)
#define VarBitsBGetAlign(vb)    { (vb).bitlen=0; }
#define VarBitsBPutSize(vb)    ((U32)( (((char*)(vb).cur)-((char*)(vb).init))*8 +(vb).bitlen ))    // in bits !!
#define VarBitsBGetSize(vb)    ((U32)( (((char*)(vb).cur)-((char*)(vb).init))*8 -(vb).bitlen ))    // in bits !!

// CB : stupid & with VarBitsLens just for len = 0 !
//    this is because >>32 = NOP !! (on x86 anyway)
#define VarBitsBGet(v,typ,vb,len)                                 \
{                                                                 \
  if (((vb).bitlen)<(len)) {                                      \
    register BITSTYPE nb=*((BITSTYPE* RADRESTRICT)((vb).cur));    \
    v=(typ)((((vb).bits)|(nb>>((vb).bitlen)))>>(BITSTYPELEN-len));\
    ((vb).bits)=nb<<((len)-((vb).bitlen));                        \
    ((vb).bitlen)=((vb).bitlen)+BITSTYPELEN-(len);                \
    ((vb).cur)=((char*)((vb).cur))+BITSTYPEBYTES;                 \
  } else {                                                        \
    v=(typ)(((vb).bits)>>(BITSTYPELEN-len));                      \
    ((vb).bits)<<=(len);                                          \
    ((vb).bitlen)-=(len);                                         \
    v &= (len)?0xffffffff:0;                                      \
  }                                                               \
}

#define VarBitsBPeek(v,typ,vb,len)                                \
{                                                                 \
  if (((vb).bitlen)<(len)) {                                      \
    register BITSTYPE nb=*((BITSTYPE* RADRESTRICT)((vb).cur));    \
    v=(typ)((((vb).bits)|(nb>>((vb).bitlen)))>>(BITSTYPELEN-len));\
  } else {                                                        \
    v=(typ)(((vb).bits)>>(BITSTYPELEN-len));                      \
  }                                                               \
}

#define VarBitsBUse(vb,len)                                       \
{                                                                 \
  if (((vb).bitlen)<(len)) {                                      \
    register BITSTYPE nb=*((BITSTYPE* RADRESTRICT)((vb).cur));    \
    ((vb).bits)=nb<<((len)-((vb).bitlen));                        \
    ((vb).bitlen)=((vb).bitlen)+BITSTYPELEN-(len);                \
    ((vb).cur)=((char*)((vb).cur))+BITSTYPEBYTES;                 \
  } else {                                                        \
    ((vb).bits)<<=(len);                                          \
    ((vb).bitlen)-=(len);                                         \
  }                                                               \
}

// get 1 bit
//    best to use like : if ( VarBitsGet1(vb,i) )
//    or bit = VarBitsGet1(vb,temp);
// WARNING : CB: i is NOT masked by &1 in this function, you must do it yourself !
//    'temp_u32' MUST be a U32 !!
#define VarBitsBGet1(vb,temp_u32)     \
(                                     \
  ((vb).bitlen==0)?                   \
  (                                   \
    temp_u32=*((BITSTYPE* RADRESTRICT)((vb).cur)),\
    ((vb).cur)=((char*)((vb).cur))+BITSTYPEBYTES, \
    ((vb).bits)=((BITSTYPE)temp_u32)<<1,     \
    ((vb).bitlen)=(BITSTYPELEN-1)     \
  ):(                                 \
    temp_u32=((vb).bits),             \
    ((vb).bits)<<=1,                  \
    --((vb).bitlen)                   \
  ),                                  \
    (temp_u32>>(BITSTYPELEN-1))       \
)



//*
#ifdef __RADX86__
#ifdef _DEBUG
#ifdef __cplusplus

inline void _VarBitsBGet(U32 & v,VARBITSB & vb,int len)
{
    radassert( ! vb.isPut );

  if ( (int)((vb).bitlen) < (len))
  {
    register BITSTYPE nb=*((BITSTYPE* RADRESTRICT)((vb).cur));
    v=((((vb).bits)|(nb>>((vb).bitlen)))>>(BITSTYPELEN-len));
    ((vb).bits)=nb<<((len)-((vb).bitlen));
    ((vb).bitlen)=((vb).bitlen)+BITSTYPELEN-(len);
    ((vb).cur)=((char*)((vb).cur))+BITSTYPEBYTES;
    v &= VarBitsLens[len];
  }
  else
  {
    int shift = BITSTYPELEN - len;
    v=(((vb).bits)>>shift);
    ((vb).bits)<<=(len);
    ((vb).bitlen)-=(len);
    v &= VarBitsLens[len];
  }
}

#undef VarBitsBGet
#define VarBitsBGet(v,typ,vb,len) _VarBitsBGet((U32 &)v,vb,len)


// __v=(val)&VarBitsLens[__s]
inline void _VarBitsBPut(VARBITSB & vb,U32 val,int size)
{
    radassert( vb.isPut );

    S32 __s=size;
    U32 __v=(val);
    radassert( __v < (1UL<<__s) );

    if ((vb).bitlen+__s >=BITSTYPELEN)
    {
        S32 __r = __s + (vb).bitlen - BITSTYPELEN;
        radassert( __r >= 0 && __r < 32 );
        radassert( (vb).bitlen != 0 );
        (vb).bits <<= (BITSTYPELEN-(vb).bitlen);
        (vb).bits |= __v >> __r;
        *((U32*)(vb).cur)=(vb).bits;
        (vb).cur=((char*)((vb).cur)+4);
        (vb).bitlen = __r;
        (vb).bits = __v&VarBitsLens[__r];
    }
    else
    {
        radassert( __s < 32 );
        (vb).bits = ((vb).bits<<__s) | __v;
        (vb).bitlen+=__s;
    }
}

#undef VarBitsBPut
#define VarBitsBPut(vb,val,size) _VarBitsBPut(vb,val,size)

#endif // cplusplus
#endif // _DEBUG
#endif // __RADX86__
/**/


#endif
