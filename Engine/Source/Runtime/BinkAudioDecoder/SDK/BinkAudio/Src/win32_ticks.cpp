// Copyright Epic Games, Inc. All Rights Reserved.
#include "ticks.h"

#include <windows.h>

#pragma warning(push)
#pragma warning(disable:4035) // no return value

#ifdef __RADX64__

  #define our_rdtsc() __rdtsc()

#else
  static inline U64 our_rdtsc()
  { 
      __asm rdtsc;
      // eax/edx returned
  }
#endif

U64 baue_ticks( void )
{
  return our_rdtsc();
}
