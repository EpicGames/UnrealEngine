// Copyright Epic Games, Inc. All Rights Reserved.
#include "ticks.h"
#include <windows.h>

#pragma warning(push)
#pragma warning(disable:4035) // no return value

#ifdef __RADX64__

  #define our_rdtsc() __rdtsc()

#elif defined( __RADARM__ )
  static inline U64 our_rdtsc()
  { 
    U64 qpc;
    QueryPerformanceCounter( (LARGE_INTEGER*) &qpc );
    return qpc;
  }
#else
  #error What cpu?
#endif

U64 baue_ticks( void )
{
  return our_rdtsc();
}
