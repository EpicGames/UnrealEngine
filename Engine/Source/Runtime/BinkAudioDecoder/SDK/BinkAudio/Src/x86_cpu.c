// Copyright Epic Games, Inc. All Rights Reserved.
#include "cpu.h"

#if defined(__RADX86__)

RADDEFINEDATA S32 cpu_features_to_use;
RADDEFINEDATA S32 cpu_features_avail = 0;

#if defined(__GCC__) || defined(__GNUC__)
#if defined( __RADMAC__ ) || defined( __RADIPHONESIM__ )

static void __cpuid(int *info, int which)
{
    void *saved=0;

    // NOTE(fg): With PIC on Mac, can't overwrite ebx, so we
    // need to play it safe.
#ifdef __RAD64__
    __asm__ __volatile__ (
        "movq %%rbx, %5\n"
        "cpuid\n"
        "movl %%ebx, %1\n"
        "movq %5, %%rbx"
        : "=a" (info[0]), 
          "=r" (info[1]),
          "=c" (info[2]),
          "=d" (info[3])
        : "a" (which),
          "m" (saved)
        : "cc");
#else
    __asm__ __volatile__ (
        "movl %%ebx, %5\n"
        "cpuid\n"
        "movl %%ebx, %1\n"
        "movl %5, %%ebx"
        : "=a" (info[0]), 
          "=r" (info[1]),
          "=c" (info[2]),
          "=d" (info[3])
        : "a"(which),
          "m" (saved)
        : "cc");
#endif
}

#define cpuid(buf,lvl) __cpuid(buf,lvl)

#else
#include <cpuid.h>
#define cpuid(buf,lvl) __cpuid(lvl,buf[0],buf[1],buf[2],buf[3])
#endif
#else
#include <intrin.h>
#define cpuid __cpuid
#endif

static S32 CPU_available( void )
{
  S32 retval = 0;

  int info[4];
  int nIds;//, nExIds;

  cpuid(info, 0);
  nIds = info[0];

  //cpuid(info, 0x80000000);  // we need no extra stuff
  //nExIds = info[0];

  //  Detect Instruction Set
  if (nIds >= 1)
  {
      cpuid(info,0x00000001);
      if ( ( info[3] & ((int)1 << 23)) != 0 )
        retval |= CPU_MMX;
      if ( ( info[3] & ((int)1 << 25)) != 0 )
        retval |= CPU_SSE;
      if ( ( info[3] & ((int)1 << 26)) != 0 )
        retval |= CPU_SSE2;
      if ( ( info[2] & ((int)1 << 0)) != 0 )
        retval |= CPU_SSE3;
      if ( ( info[2] & ((int)1 << 9)) != 0 )
        retval |= CPU_SSSE3;
  }

  return( retval );
}

RADDEFFUNC void CPU_check( S32 use, S32 dont_use )
{
  if ( cpu_features_avail == 0 )
  {
    cpu_features_avail = CPU_available() | 0x80000000;
    cpu_features_to_use = cpu_features_avail;
  }

  cpu_features_to_use |= ( cpu_features_avail & use );

  cpu_features_to_use &= ~( cpu_features_avail & dont_use );
}

#include <mmintrin.h>

RADDEFFUNC void CPU_clear( void )
{
#ifndef __RAD64__ // in 64-bit mode, we don't use MMX.
  _mm_empty();
#endif
}

#endif // __RADX86__